[CoreRedirects]
+ClassRedirects=(OldName="/Script/LiveLinkHub.LiveLinkHubSessionExtraData",NewName="/Script/LiveLinkInterface.LiveLinkHubSessionExtraData")