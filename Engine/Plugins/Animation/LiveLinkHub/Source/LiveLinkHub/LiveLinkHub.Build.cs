// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildTool;

public class LiveLinkHub : ModuleRules
{
	public LiveLinkHub(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"LiveLinkInterface" // Can be removed in 5.10. This dependency was added because some headers were moved to LiveLinkInterface module.
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetRegistry",
				"AssetTools",
				"ContentBrowser",
				"ContentBrowserAssetDataSource",
				"ContentBrowserData",
				"EditorStyle",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"LiveLink",
				"LiveLinkEditor",
				"LiveLinkHubMessaging",
				"LiveLinkInterface",
				"LiveLinkMessageBusFramework",
				"MediaAssets",
				"MediaIOCore",
				"MediaProfile",
				"Messaging",
				"ProjectSettingsViewer",
				"SharedSettingsWidgets",
				"Settings",
				"SettingsEditor",
				"Slate",
				"SlateCore",
				"StatusBar",
				"TimeManagement",
				"ToolMenus",
				"ToolWidgets",
				"TraceLog",
				"UnrealEd",
				"OutputLog",
				"NamingTokens",
				"MessageBusTesterEditor"
			});

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				});

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				});

			if (Target.bBuildTargetDeveloperTools)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"SessionServices",
						"SessionFrontend",
					});
			}
		}
	}
}
