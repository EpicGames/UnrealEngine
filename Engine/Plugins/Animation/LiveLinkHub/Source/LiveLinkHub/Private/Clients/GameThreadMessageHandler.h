// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageHandler.h"

#include "IMessageContext.h"
#include "Templates/SharedPointer.h"

/**
 * Template for handlers that should handle their message on the game thread.
 *
 * @param MessageType The type of message to handle.
 * @param HandlerType The type of the handler class.
 */
template<typename MessageType, typename HandlerType>
class TGameThreadMessageHandler
	: public IMessageHandler
{
public:

	/** Type definition for function pointers that are compatible with this TGameThreadMessageHandler. */
	typedef void (HandlerType::* FuncType)(const MessageType&, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&);

public:

	/**
	 * Creates and initializes a new message handler.
	 *
	 * @param InHandler The object that will handle the messages.
	 * @param InFunc The object's message handling function.
	 */
	TGameThreadMessageHandler(HandlerType* InHandler, FuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(InHandler != nullptr);
	}

	/** Virtual destructor. */
	~TGameThreadMessageHandler() = default;

public:

	//~ IMessageHandler interface

	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		TWeakPtr<HandlerType> WeakHandler = Handler->AsShared();

		auto DelegatedMethod = [WeakHandler, Context, FuncPtr = Func]()
		{
			if (TSharedPtr<HandlerType> HandlerPin = WeakHandler.Pin())
			{
				((HandlerPin.Get())->*FuncPtr)(*static_cast<const MessageType*>(Context->GetMessage()), Context);
			}
		};

		UStruct* Struct = MessageType::StaticStruct();
		if (Struct && Context->GetMessageTypePathName() == Struct->GetStructPathName())
		{
			AsyncTask(ENamedThreads::GameThread, MoveTemp(DelegatedMethod));
		}
	}

private:

	/** Holds a pointer to the object handling the messages. */
	HandlerType* Handler;

	/** Holds a pointer to the actual handler function. */
	FuncType Func;
};
