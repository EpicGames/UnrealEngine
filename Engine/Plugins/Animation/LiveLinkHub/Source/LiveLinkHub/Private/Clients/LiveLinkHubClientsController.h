// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubUEClientInfo.h"
#include "LiveLinkHubProvider.h"
#include "LiveLinkTypes.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Session/LiveLinkHubSession.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "SLiveLinkHubClientsView.h"
#include "UObject/StructOnScope.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

/** Controller responsible for holding the list of connected clients and creating the clients view. */
class FLiveLinkHubClientsController
{
public:
	FLiveLinkHubClientsController(const TSharedRef<ILiveLinkHubClientsModel>& InClientsModel)
		: ClientsModel(InClientsModel)
	{
		const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		LiveLinkHubModule.GetSessionManager()->OnActiveSessionChanged().AddRaw(this, &FLiveLinkHubClientsController::OnActiveSessionChanged);

		ClientsModel->OnClientEvent().AddRaw(this, &FLiveLinkHubClientsController::OnClientEvent);
		
		UEClientInfo = MakeShared<TStructOnScope<FLiveLinkHubUEClientInfo>>();
	}

	~FLiveLinkHubClientsController()
	{
		if (const FLiveLinkHubModule* LiveLinkHubModule = FModuleManager::Get().GetModulePtr<FLiveLinkHubModule>("LiveLinkHub"))
		{
			if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = LiveLinkHubModule->GetSessionManager())
			{
				SessionManager->OnActiveSessionChanged().RemoveAll(this);
			}
		}

		ClientsModel->OnClientEvent().RemoveAll(this);
	}

	/** Create the widget that displays connected UE clients. */
	TSharedRef<SWidget> MakeClientsView()
	{
		return SAssignNew(ClientsView, SLiveLinkHubClientsView, ClientsModel.ToSharedRef())
			.OnClientSelected_Raw(this, &FLiveLinkHubClientsController::UpdateClientDetails)
			.OnDiscoveredClientPicked_Raw(this, &FLiveLinkHubClientsController::OnDiscoveredClientPicked)
			.OnRemoveClientFromSession_Raw(this, &FLiveLinkHubClientsController::OnRemoveClientFromSession);
	}

	/** Create the widget that displays information about a given UE client. */
	TSharedRef<SWidget> MakeClientDetailsView()
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = NAME_None;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = true;

		FStructureDetailsViewArgs StructDetailsArgs;
		StructDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructDetailsArgs, UEClientInfo);
		StructDetailsView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda([](const FPropertyAndParent&){ return true ;}));
		
		return StructDetailsView->GetWidget().ToSharedRef();
	}

private:
	/** Update the client info details if we received new info about it. */
	void OnClientEvent(FLiveLinkHubClientId ClientIdentifier, ILiveLinkHubClientsModel::EClientEventType EventType)
	{
		if (!ClientsView)
		{
			return;
		}
		switch (EventType)
		{
			case ILiveLinkHubClientsModel::EClientEventType::Connected:
			{
				
				ClientsView->Reinitialize();
				break;
			}
			case ILiveLinkHubClientsModel::EClientEventType::Reestablished:
				// fallthrough
			case ILiveLinkHubClientsModel::EClientEventType::Modified:
			{
				TArray<FLiveLinkHubClientId> ClientIds = ClientsView->GetSelectedClients();

				FLiveLinkHubClientId ClientId;
				if (ClientIds.Num() > 0)
				{
					ClientId = ClientIds[ClientIds.Num() - 1];
				}

				UpdateClientDetails(ClientId);
				break;
			}
		}
	}

	void OnActiveSessionChanged(const TSharedRef<ILiveLinkHubSession>& ActiveSession)
	{
		if (ClientsView)
		{
			ClientsView->Reinitialize();
		}

		StructDetailsView->SetStructureData(nullptr);
	}
	
	/** Handles updating client details in the client details panel. */
	void UpdateClientDetails(FLiveLinkHubClientId Client)
	{
		if (!Client.IsValid())
		{
			Client = DisplayedClientId;
		}

		if (TOptional<FLiveLinkHubUEClientInfo> ClientInfo = ClientsModel->GetClientInfo(Client))
		{
			DisplayedClientId = Client;

			UEClientInfo->InitializeAs<FLiveLinkHubUEClientInfo>(*ClientInfo);
			if (StructDetailsView)
			{
				StructDetailsView->SetStructureData(UEClientInfo);
			}
		}
		else
		{
			// Disable this for the time being, this hides the details panel when you click on the empty list
			//StructDetailsView->SetStructureData(nullptr);
		}
	}

	void OnDiscoveredClientPicked(FLiveLinkHubClientId Client)
	{
		ClientsModel->ConnectTo(Client);
	}
	
	void OnRemoveClientFromSession(FLiveLinkHubClientId Client)
	{
		if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkHub()->GetSessionManager())
		{
			if (TSharedPtr<ILiveLinkHubSession> CurrentSession = SessionManager->GetCurrentSession())
			{
				if (Client == DisplayedClientId)
				{
					if (UEClientInfo)
					{
						// Clear client details if it was selected.
						UEClientInfo->Reset();
						StructDetailsView->SetStructureData(nullptr);
					}
				}

				CurrentSession->RemoveClient(Client);
			}
		}
	}

private:
	/** List view of UE clients the hub can connect to. */
	TSharedPtr<SLiveLinkHubClientsView> ClientsView;
	/** Ptr to the livelink hub. */
	TSharedPtr<ILiveLinkHubClientsModel> ClientsModel;
	/** Holds a struct representing information about the currently selected client. */
	TSharedPtr<TStructOnScope<FLiveLinkHubUEClientInfo>> UEClientInfo;
	/** ID of the client displayed in the panel. */
	FLiveLinkHubClientId DisplayedClientId;
	/** Holds the details view created with the property editor module. */
	TSharedPtr<IStructureDetailsView> StructDetailsView;
	/** Delegate called when one of the connection's info has changed. */
	FDelegateHandle ConnectionStatusChangedHandle;
};
