// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubProvider.h"

#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "Async/Async.h"
#include "Clients/LiveLinkHubUEClientInfo.h"
#include "Containers/ObservableArray.h"
#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Editor.h"
#include "HAL/CriticalSection.h"
#include "INetworkMessagingExtension.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkHubMessagingSettings.h"
#include "LiveLinkSettings.h"
#include "Logging/StructuredLog.h"
#include "MessageEndpointBuilder.h"
#include "Misc/ScopeLock.h"
#include "Session/LiveLinkHubSession.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "Settings/LiveLinkHubSettings.h"
#include "Settings/LiveLinkHubTimingUtils.h"
#include "TimerManager.h"


#define LOCTEXT_NAMESPACE "LiveLinkHub.LiveLinkHubProvider"

namespace LiveLinkHubProviderUtils
{
	static INetworkMessagingExtension* GetMessagingStatistics()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		if (IsInGameThread())
		{
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		else
		{
			IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;

			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}

		ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
		return nullptr;
	}

	FString GetIPAddress(const FMessageAddress& ClientAddress)
	{
		FString IPAddress;
		if (INetworkMessagingExtension* Statistics = GetMessagingStatistics())
		{
			const FGuid NodeId = Statistics->GetNodeIdFromAddress(ClientAddress);
			IPAddress = NodeId.IsValid() ? Statistics->GetLatestNetworkStatistics(NodeId).IPv4AsString : FString();

			int32 PortIndex = INDEX_NONE;
			IPAddress.FindChar(TEXT(':'), PortIndex);

			// Cut off the port from the end.
			if (PortIndex != INDEX_NONE)
			{
				IPAddress.LeftInline(PortIndex);
			}
		}
		return IPAddress;
	}
}


FLiveLinkHubProvider::FLiveLinkHubProvider(const TSharedRef<ILiveLinkHubSessionManager>& InSessionManager, const FString& InProviderName)
	: FLiveLinkProvider(InProviderName, false)
	, SessionManager(InSessionManager)
{
	Annotations.Add(FLiveLinkHubMessageAnnotation::ProviderTypeAnnotation, UE::LiveLinkHub::Private::LiveLinkHubProviderType.ToString());

	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(*GetProviderName());
	EndpointBuilder.WithHandler(MakeHandler<FLiveLinkClientInfoMessage>(&FLiveLinkHubProvider::HandleClientInfoMessage));
	EndpointBuilder.WithHandler(MakeHandler<FLiveLinkHubConnectMessage>(&FLiveLinkHubProvider::HandleHubConnectMessage));
	EndpointBuilder.WithHandler(MakeHandler<FLiveLinkHubDisconnectMessage>(&FLiveLinkHubProvider::HandleHubDisconnectMessage));
	EndpointBuilder.WithHandler(MakeHandler<FLiveLinkHubBeaconMessage>(&FLiveLinkHubProvider::HandleBeaconMessage));

	CreateMessageEndpoint(EndpointBuilder);

	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{
		const double ValidateConnectionsRate = GetDefault<ULiveLinkSettings>()->MessageBusPingRequestFrequency;
		if (GEditor)
		{
			GEditor->GetTimerManager()->SetTimer(ValidateConnectionsTimer, FTimerDelegate::CreateRaw(this, &FLiveLinkHubProvider::ValidateHubConnections), ValidateConnectionsRate, true);
		}

		GetMutableDefault<ULiveLinkHubUserSettings>()->OnFiltersModifiedDelegate.AddRaw(this, &FLiveLinkHubProvider::OnFiltersModified);
	});

	IModularFeatures::Get().RegisterModularFeature(ILiveLinkHubClientsModel::GetModularFeatureName(), this);
}

FLiveLinkHubProvider::~FLiveLinkHubProvider()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		GetMutableDefault<ULiveLinkHubUserSettings>()->OnFiltersModifiedDelegate.RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(ValidateConnectionsTimer);
	}

	IModularFeatures::Get().UnregisterModularFeature(ILiveLinkHubClientsModel::GetModularFeatureName(), this);
}

bool FLiveLinkHubProvider::ShouldTransmitToSubject_AnyThread(FName SubjectName, FMessageAddress Address) const
{
	if (!Address.IsValid())
	{
		return false;
	}

	FLiveLinkHubClientId ClientId;
	{
		UE::TReadScopeLock Locker(ClientsMapLock);
		ClientId = AddressToIdCache.FindRef(Address);
	}

	return ShouldTransmitToClient_AnyThread(ClientId, SubjectName);
}

void FLiveLinkHubProvider::UpdateTimecodeSettings(const FLiveLinkHubTimecodeSettings& InSettings, const FLiveLinkHubClientId& ClientId)
{
	SendTimecodeSettings(InSettings, ClientId);
}

void FLiveLinkHubProvider::ResetTimecodeSettings(const FLiveLinkHubClientId& ClientId)
{
	// Sending settings with ELiveLinkHubTimecodeSource::NotDefined will reset the timecode on the client.
	SendTimecodeSettings(FLiveLinkHubTimecodeSettings{}, ClientId);
}

void FLiveLinkHubProvider::UpdateCustomTimeStepSettings(const FLiveLinkHubCustomTimeStepSettings& InSettings, const FLiveLinkHubClientId& ClientId)
{
	SendCustomTimeStepSettings(InSettings, ClientId);
}

void FLiveLinkHubProvider::UpdateSourceEvaluationMode(ELiveLinkSourceMode Mode, const FLiveLinkHubClientId& ClientId)
{
	SendSourceEvaluationMode(Mode, ClientId);
}

void FLiveLinkHubProvider::ResetCustomTimeStepSettings(const FLiveLinkHubClientId& ClientId)
{
	// Setting the bResetCustomTimeStep flag will reset the CustomTimeStep on the client.
	FLiveLinkHubCustomTimeStepSettings ResetCustomTimeStepSettings;
	ResetCustomTimeStepSettings.bResetCustomTimeStep = true;

	SendCustomTimeStepSettings(ResetCustomTimeStepSettings, ClientId);
}

void FLiveLinkHubProvider::DisconnectAll()
{
	UE_LOGF(LogLiveLinkHub, Verbose, "Provider (%ls): Sending a disconnect message to all DiscoveryV1 clients", *GetEndpointAddress().ToString());

	TArray<FLiveLinkHubAddressBookEntry> ClientsToDisconnect;
	{
		UE::TReadScopeLock Lock(AddressBookLock);
		ClientsToDisconnect = AddressBook.FilterByPredicate([](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.bConnected; });
	}

	for (const FLiveLinkHubAddressBookEntry& EntryToDisconnect : ClientsToDisconnect)
	{
		DisconnectClient(EntryToDisconnect);
	}
}

void FLiveLinkHubProvider::DisconnectClient(const FLiveLinkHubClientId& Client)
{
	if (ClientBeingDisconnected == Client)
	{
		// Don't send a disconnect message to a client being disconneted.
		return;
	}

	TOptional<FLiveLinkHubAddressBookEntry> EntryToDisconnect;
	{
		UE::TReadScopeLock Lock(AddressBookLock);
		if (FLiveLinkHubAddressBookEntry* FoundEntry = Algo::FindBy(AddressBook, Client, &FLiveLinkHubAddressBookEntry::ClientId))
		{
			EntryToDisconnect = *FoundEntry;
		}
	}

	if (EntryToDisconnect)
	{
		DisconnectClient(*EntryToDisconnect);
	}
	else
	{
		// This might have been a restored client that hasn't been updated, so clear it. 
		UE::TWriteScopeLock Locker(ClientsMapLock);
		ClientsMap.Remove(Client);

		for (auto It = AddressToIdCache.CreateIterator(); It; ++It)
		{
			if (It->Value == Client)
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FLiveLinkHubProvider::DisconnectClient(const FLiveLinkHubAddressBookEntry& EntryToDisconnect)
{
	FLiveLinkHubDisconnectMessage DisconnectMessage;
	DisconnectMessage.ProviderName = GetProviderName();
	DisconnectMessage.MachineName = GetMachineName();
	DisconnectMessage.SourceGuid = EntryToDisconnect.SourceGuid;

	// Then modify the entry in our address book.
	TOptional<FLiveLinkHubAddressBookEntry> AddressBookEntry;
	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		if (FLiveLinkHubAddressBookEntry* EntryPtr = AddressBook.FindByPredicate([&EntryToDisconnect](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ClientId == EntryToDisconnect.ClientId; }))
		{
			EntryPtr->bConnected = false;
			EntryPtr->bDisableAutoconnect = true;
			AddressBookEntry = *EntryPtr;
		}
	}
	
	// We might want to revisit this, but at the moment, we are only sending out an explicit disconnect message for V2 clients.
	// V1 Clients are disabled but not explicitly disconnected.
	if (AddressBookEntry && AddressBookEntry->DiscoveryProtocolVersion > 1)
	{
		// Send connection close message
		UE_LOGF(LogLiveLinkHub, Verbose, "Provider: Sending Disconnect message from %ls to %ls", *GetEndpointAddress().ToString(), *EntryToDisconnect.ControlEndpoint.ToString());
		SendMessage(FMessageEndpoint::MakeMessage<FLiveLinkHubDisconnectMessage>(MoveTemp(DisconnectMessage)), EntryToDisconnect.ControlEndpoint, EMessageFlags::Reliable);
		CloseConnection(EntryToDisconnect.DataEndpoint);
	}
}

void FLiveLinkHubProvider::SendTimecodeSettings(const FLiveLinkHubTimecodeSettings& InSettings, const FLiveLinkHubClientId& ClientId)
{
	if (ClientId.IsValid())
	{
		FMessageAddress RelevantControlEndpoint;

		{
			UE::TReadScopeLock Lock(AddressBookLock);
			if (const FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, ClientId, &FLiveLinkHubAddressBookEntry::ClientId))
			{
				if (Entry->bConnected)
				{
					RelevantControlEndpoint = Entry->ControlEndpoint;
				}
			}
		}


		if (RelevantControlEndpoint.IsValid())
		{
			SendMessage(FMessageEndpoint::MakeMessage<FLiveLinkHubTimecodeSettings>(InSettings), RelevantControlEndpoint, EMessageFlags::Reliable);
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not find address for client %ls.", *ClientId.ToString());
		}
	}
	else
	{
		// Invalid ID means we're broadcasting to all connected clients.
		SendControlMessageToEnabledClients(FMessageEndpoint::MakeMessage<FLiveLinkHubTimecodeSettings>(InSettings));
	}
}

void FLiveLinkHubProvider::SendCustomTimeStepSettings(const FLiveLinkHubCustomTimeStepSettings& InSettings, const FLiveLinkHubClientId& ClientId)
{
	if (ClientId.IsValid())
	{
		FMessageAddress RelevantControlEndpoint;

		{
			UE::TReadScopeLock Lock(AddressBookLock);
			if (const FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, ClientId, &FLiveLinkHubAddressBookEntry::ClientId))
			{
				if (Entry->bConnected)
				{
					RelevantControlEndpoint = Entry->ControlEndpoint;
				}
			}
		}

		if (RelevantControlEndpoint.IsValid())
		{
			SendMessage(FMessageEndpoint::MakeMessage<FLiveLinkHubCustomTimeStepSettings>(InSettings), RelevantControlEndpoint, EMessageFlags::Reliable);
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not find address for client %ls.", *ClientId.ToString());
		}
	}
	else
	{
		// Invalid ID means we're broadcasting to all clients.
		SendControlMessageToEnabledClients(FMessageEndpoint::MakeMessage<FLiveLinkHubCustomTimeStepSettings>(InSettings), EMessageFlags::Reliable);
	}
}

void FLiveLinkHubProvider::SendSourceEvaluationMode(ELiveLinkSourceMode Mode, const FLiveLinkHubClientId& ClientId)
{
	if (ClientId.IsValid())
	{
		FMessageAddress RelevantControlEndpoint;
		FGuid RelevantSourceGuid;

		{
			UE::TReadScopeLock Lock(AddressBookLock);
			if (const FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, ClientId, &FLiveLinkHubAddressBookEntry::ClientId))
			{
				if (Entry->bConnected)
				{
					RelevantControlEndpoint = Entry->ControlEndpoint;
					RelevantSourceGuid = Entry->SourceGuid;
				}
			}
		}

		if (RelevantControlEndpoint.IsValid())
		{
			FLiveLinkHubSourceEvaluationModeMessage* Payload = FMessageEndpoint::MakeMessage<FLiveLinkHubSourceEvaluationModeMessage>();
			Payload->Mode = Mode;
			Payload->SourceGuid = RelevantSourceGuid;
			SendMessage(Payload, RelevantControlEndpoint, EMessageFlags::Reliable);
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not find address for client %ls.", *ClientId.ToString());
		}
	}
	else
	{
		// Broadcast — leave SourceGuid invalid so the client applies the mode to all hub sources it owns.
		FLiveLinkHubSourceEvaluationModeMessage* Payload = FMessageEndpoint::MakeMessage<FLiveLinkHubSourceEvaluationModeMessage>();
		Payload->Mode = Mode;
		SendControlMessageToEnabledClients(Payload, EMessageFlags::Reliable);
	}
}

void FLiveLinkHubProvider::AddRestoredClient(FLiveLinkHubUEClientInfo& RestoredClientInfo)
{
	// If a client was already discovered with the same hostname, update it to match the restored client if it's in the address book
	bool bMatchedExistingConnection = false;

	if (const TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
	{
		if (const TSharedPtr<ILiveLinkHubSession> ActiveSession = Manager->GetCurrentSession())
		{
			TOptional<FLiveLinkHubAddressBookEntry> FoundClient;
			{
				// todo: validate this logic for a client that's already connected.
				UE::TWriteScopeLock Lock(AddressBookLock);
				for (FLiveLinkHubAddressBookEntry& Entry : AddressBook)
				{
					// todo: Multiple clients with same hostname, discriminate by project
					if (Entry.DiscoveryProtocolVersion > 1 && Entry.Hostname == RestoredClientInfo.Hostname && !Entry.bConnected)
					{
						Entry.ClientId = RestoredClientInfo.Id;
						FoundClient = Entry;
						break;
					}
				}
			}

			if (FoundClient)
			{
				UE::TWriteScopeLock Locker(ClientsMapLock);
				ClientsMap.FindOrAdd(RestoredClientInfo.Id) = RestoredClientInfo;

				SendMessage(MakeLiveLinkHubDiscoveryMessage(), FoundClient->ControlEndpoint, EMessageFlags::Reliable, GetAnnotations());
				UE_LOGF(LogLiveLinkHub, Log, "Restoring connection to client %ls (%ls %ls)", *FoundClient->ClientId.ToString(), *FoundClient->Hostname, *FoundClient->IP);
			}
			else
			{
				// If we haven't found a v2 client for this entry, fallback to previous method.
				UE::TReadScopeLock Locker(ClientsMapLock);

				for (auto It = ClientsMap.CreateIterator(); It; ++It)
				{
					FLiveLinkHubUEClientInfo& IteratedClient = It->Value;
					if (IteratedClient.Hostname == RestoredClientInfo.Hostname && !ActiveSession->IsClientInSession(It->Key))
					{
						bMatchedExistingConnection = true;

						// Update Client info from the new connection.
						// todo DiscoveryProtocolV2, do we need to override the ID in this case? 
						RestoredClientInfo = It->Value;
						break;
					}
				}
			}
		}
	}

	if (!bMatchedExistingConnection)
	{
		ClientsMap.FindOrAdd(RestoredClientInfo.Id) = RestoredClientInfo;
	}

	// todo: Revisit this, we should save an IP filter in the session 
	// OnClientEventDelegate.Broadcast(RestoredClientInfo.Id, EClientEventType::Discovered);
}

TOptional<FLiveLinkHubUEClientInfo> FLiveLinkHubProvider::GetClientInfo(FLiveLinkHubClientId InClient) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	TOptional<FLiveLinkHubUEClientInfo> ClientInfo;
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(InClient))
	{
		ClientInfo = *ClientInfoPtr;
	}

	return ClientInfo;
}

void FLiveLinkHubProvider::HandleHubConnectMessage(const FLiveLinkHubConnectMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogLiveLinkHub, Verbose, "Provider: Received connect message from %ls", *Context->GetSender().ToString());

	const ELiveLinkTopologyMode Mode = FLiveLinkHub::Get()->GetTopologyMode();
	if (!GetDefault<ULiveLinkHubMessagingSettings>()->CanTransmitTo(Mode, Message.ClientInfo.TopologyMode))
	{
		UE_LOGF(LogLiveLinkHub, Verbose, "Provider: Denying connection from %ls since its mode is incompatible with this provider's.", *Context->GetSender().ToString());
		FLiveLinkHubDisconnectMessage DisconnectMessage{ GetProviderName(), GetMachineName() };
		SendMessage(FMessageEndpoint::MakeMessage<FLiveLinkHubDisconnectMessage>(MoveTemp(DisconnectMessage)), Context->GetSender(), EMessageFlags::Reliable);
		CloseConnection(Context->GetSender());
		return;
	}

	const FMessageAddress DataEndpoint = Context->GetSender();

	FLiveLinkConnectMessage ConnectMessage;
	ConnectMessage.LiveLinkVersion = Message.ClientInfo.LiveLinkVersion;

	// Get the address to the messagebus source endpoint.
	FMessageAddress ControlEndpoint;
	FMessageAddress::Parse(Message.ControlEndpoint, ControlEndpoint);
	if (!ControlEndpoint.IsValid())
	{
		// Backwards compatibile if connecting to a client built before 5.7.
		ControlEndpoint = DataEndpoint;
	}

	const int32 DiscoveryProtocolVersion = ControlEndpoint != DataEndpoint ? 2 : 1;

	FLiveLinkProvider::HandleConnectMessage(ConnectMessage, Context);

	TOptional<FLiveLinkHubClientId> UpdatedClient;
	{
		UE::TWriteScopeLock Locker(ClientsMapLock);

		// First check if there are multiple disconnected entries with the same host as the incoming client.
		uint8 NumClientsForHost = 0;
		for (auto It = ClientsMap.CreateIterator(); It; ++It)
		{
			FLiveLinkHubUEClientInfo& IteratedClient = It->Value;
			if (IteratedClient.Hostname == Message.ClientInfo.Hostname && IteratedClient.Status == ELiveLinkClientStatus::Disconnected)
			{
				NumClientsForHost++;
				if (NumClientsForHost > 1)
				{
					break;
				}
			}
		}

		// Remove old entries if one is found
		for (auto It = ClientsMap.CreateIterator(); It; ++It)
		{
			FLiveLinkHubUEClientInfo& IteratedClient = It->Value;
			// If there are multiple disconnected clients with the same hostname, try finding a client with the same project.
			bool bFindWithMatchingProject = NumClientsForHost > 1;

			// Only replace disconnected clients to support multiple UE instances on the same host.
			if (IteratedClient.Status == ELiveLinkClientStatus::Disconnected && IteratedClient.Hostname == Message.ClientInfo.Hostname)
			{
				if (!bFindWithMatchingProject || IteratedClient.ProjectName == Message.ClientInfo.ProjectName)
				{
					IteratedClient.UpdateFromInfoMessage(Message.ClientInfo);
					IteratedClient.Id = It->Key;
					IteratedClient.Status = ELiveLinkClientStatus::Connected;

					AddressToIdCache.FindOrAdd(DataEndpoint) = IteratedClient.Id;

					UpdatedClient = IteratedClient.Id;
					break;
				}
            }
        }
	}

	if (UpdatedClient)
	{
		// A client was just restored, make sure to update its address book entry.
		{
			UE::TWriteScopeLock Lock(AddressBookLock);
			if (FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, ControlEndpoint, &FLiveLinkHubAddressBookEntry::ControlEndpoint))
			{
				Entry->bConnected = true;
				Entry->ClientId = *UpdatedClient;
			}
		}

		OnClientEventDelegate.Broadcast(*UpdatedClient, EClientEventType::Reestablished);
	}
	else
	{
		// Actually added a new entry in the map.

		// ClientID was set at discover time, so let's find it.
		FLiveLinkHubClientId ClientId;

		{
			UE::TWriteScopeLock Lock(AddressBookLock);

			if (FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, ControlEndpoint, &FLiveLinkHubAddressBookEntry::ControlEndpoint))
			{
				Entry->bConnected = true;
				Entry->SourceGuid = Message.SourceGuid;
				Entry->DataEndpoint = DataEndpoint;
				ClientId = Entry->ClientId;
			}
		}

		if (!ClientId.IsValid())
		{
			ClientId = UpdateAddressBookEntry(ControlEndpoint, DiscoveryProtocolVersion, Message.ClientInfo.TopologyMode, Message.ClientInfo.Hostname, Message.ClientInfo.ProjectName);

			if (DiscoveryProtocolVersion > 1)
			{
				UE_LOGF(LogLiveLinkHub, Warning, "Connecting to client %ls with DiscoveryProtocolV2 that wasn't previously discovered.", *ControlEndpoint.ToString());
			}
		}

		FLiveLinkHubUEClientInfo NewClient{Message.ClientInfo, ClientId };
		NewClient.IPAddress = LiveLinkHubProviderUtils::GetIPAddress(ControlEndpoint);

		UpdatedClient = ClientId; // Set this so we can update the client's timecode provider down below.

		{
			UE::TWriteScopeLock Locker(ClientsMapLock);
			AddressToIdCache.FindOrAdd(DataEndpoint) = ClientId;
			ClientsMap.Add(NewClient.Id, MoveTemp(NewClient));
		}

		const bool bSameHost = Message.ClientInfo.Hostname == GetMachineName();
		// Note: In DiscoveryProtocolV2, if we've received a connect message then it means we've already went through the auto-connect filter, and this is an explicit connect request.
		if (ShouldAutoConnectTo(Message.ClientInfo.Hostname) || DiscoveryProtocolVersion > 1)
		{
			// Todo: At the moment we have to do this to avoid firing the session delegate outside the game thread.
			AsyncTask(ENamedThreads::GameThread, [WeakSessionManager = SessionManager, ClientId]()
				{
					if (const TSharedPtr<ILiveLinkHubSessionManager> Manager = WeakSessionManager.Pin())
					{
						if (const TSharedPtr<ILiveLinkHubSession> CurrentSession = Manager->GetCurrentSession())
						{
							CurrentSession->AddClient(ClientId);
						}
					}
				});
		}
		else
		{
			// todo: Verify if this is needed for older clients.
			OnClientEventDelegate.Broadcast(ClientId, EClientEventType::Discovered);
		}
	}

	// Update the timecode provider when a client establishes connection.
	if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsTimecodeSource)
	{
		SendTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile(), *UpdatedClient);
	}

	if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsCustomTimeStepSource)
	{
		SendCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile(), *UpdatedClient);
	}

	// Push the hub's preferred evaluation mode so newly-connected clients adopt it immediately.
	SendSourceEvaluationMode(GetDefault<ULiveLinkHubTimeAndSyncSettings>()->PreferredClientEvaluationMode, *UpdatedClient);
}

void FLiveLinkHubProvider::HandleClientInfoMessage(const FLiveLinkClientInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogLiveLinkHub, Verbose, "Provider: Received ClientInfo message from %ls", *Context->GetSender().ToString());

	FMessageAddress Address = Context->GetSender();

	FLiveLinkHubClientId ClientId;
	{
		UE::TReadScopeLock Lock(AddressBookLock);

		if (const FLiveLinkHubAddressBookEntry* Entry = Algo::FindBy(AddressBook, Context->GetSender(), &FLiveLinkHubAddressBookEntry::ControlEndpoint))
		{
			ClientId = Entry->ClientId;
		}
	}

	{
		UE::TWriteScopeLock Locker(ClientsMapLock);
		if (FLiveLinkHubUEClientInfo* ClientInfo = ClientsMap.Find(ClientId))
		{
			ClientInfo->UpdateFromInfoMessage(Message);
		}
	}

	if (ClientId.IsValid())
	{
		if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsTimecodeSource)
		{
			SendTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile(), ClientId);
		}

		if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsCustomTimeStepSource)
		{
			SendCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile(), ClientId);
		}

		SendSourceEvaluationMode(GetDefault<ULiveLinkHubTimeAndSyncSettings>()->PreferredClientEvaluationMode, ClientId);

		OnClientEventDelegate.Broadcast(ClientId, EClientEventType::Modified);
	}
}

void FLiveLinkHubProvider::HandleHubDisconnectMessage(const FLiveLinkHubDisconnectMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogLiveLinkHub, Verbose, "Provider: Received disconnect message from %ls", *Context->GetSender().ToString());

	// Received a disconnect message from the livelinkhub source (it was probably deleted), so let's remove this client from the session
	const FMessageAddress Address = Context->GetSender();

	FLiveLinkHubClientId ClientId;
	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		if (FLiveLinkHubAddressBookEntry* AddressBookEntry = AddressBook.FindByPredicate([&Address](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ControlEndpoint == Address; }))
		{
			AddressBookEntry->bConnected = false;

			/* We may want to eventually differentiate between a client disconnecting from a shutdown vs an explicit removal of the LLH source. 
				We would only want to disable autoconnect for the latter.*/
			AddressBookEntry->bDisableAutoconnect = true;
			ClientId = AddressBookEntry->ClientId;
		}
	}

	if (!ClientId.IsValid())
	{
		UE_LOGF(LogLiveLinkHub, Warning, "Attempting to disconnect client %ls with empty client id.", *Address.ToString());
		return;
	}

	// Disabling it, so reset its timecode and custom time step.
	if (TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
	{
		// The session will handle resetting the timecode settings and removing it from the cache.
		// Note: This prevents removing a client that's already in the process of being disconnected.
		TGuardValue<TOptional<FLiveLinkHubClientId>> DisconnectingClient(ClientBeingDisconnected, ClientId);
		Manager->GetCurrentSession()->RemoveClient(ClientId);
	}
}

bool FLiveLinkHubProvider::ShouldTransmitToClient_AnyThread(const FLiveLinkHubClientId& ClientId, FLiveLinkSubjectName SubjectName) const
{
	if (!IsSubjectEnabled(ClientId, SubjectName))
	{
		return false;
	}

	UE::TReadScopeLock Locker(ClientsMapLock);
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(ClientId))
	{
		if (const TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
		{
			if (!ClientInfoPtr->bEnabled)
			{
				return false;
			}

			if (const TSharedPtr<ILiveLinkHubSession> CurrentSession = Manager->GetCurrentSession())
			{
				return CurrentSession->IsClientInSession(ClientInfoPtr->Id);
			}
		}
	}

	return false;
}

void FLiveLinkHubProvider::ConnectToAllDiscoveredClients()
{
	TArray<FMessageAddress> CompatibleClients;

	{
		UE::TReadScopeLock Lock(AddressBookLock);
		Algo::TransformIf(AddressBook, CompatibleClients,
			[](const FLiveLinkHubAddressBookEntry& Entry) { return GetDefault<ULiveLinkHubMessagingSettings>()->CanTransmitTo(FLiveLinkHub::Get()->GetTopologyMode(), Entry.TopologyMode); },
			[](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ControlEndpoint; });
	}

	// For Discovery Protocol V1
	Publish(MakeLiveLinkHubDiscoveryMessage(), GetAnnotations());

	SendMessage(MakeLiveLinkHubDiscoveryMessage(), CompatibleClients, EMessageFlags::Reliable, GetAnnotations());
}

void FLiveLinkHubProvider::OnConnectionsClosed(const TArray<FMessageAddress>& ClosedAddresses)
{
	CloseConnections(ClosedAddresses);

	for (FMessageAddress TrackedAddress : ClosedAddresses)
	{
		UE::TWriteScopeLock Locker(ClientsMapLock);
		if (FLiveLinkHubClientId* ClientId = AddressToIdCache.Find(TrackedAddress))
		{
			// Removing this might have implications for restoring sessions.
			// We could instead remove this when the connection is forcibly closed.
			ClientsMap.Remove(*ClientId);
		}

		AddressToIdCache.Remove(TrackedAddress);
	}
}

void FLiveLinkHubProvider::CloseConnections(const TArray<FMessageAddress>& ClosedAddresses)
{
	// List of OldId -> NewId
	TArray<FLiveLinkHubClientId> Notifications;
	TArray<FMessageAddress> AddressesToRemove;
	{
		UE::TWriteScopeLock Locker(ClientsMapLock);

		for (FMessageAddress TrackedAddress : ClosedAddresses)
		{
			FLiveLinkHubClientId ClientId = AddressToIdCache.FindRef(TrackedAddress);
			if (FLiveLinkHubUEClientInfo* FoundInfo = ClientsMap.Find(ClientId))
			{
				FoundInfo->Status = ELiveLinkClientStatus::Disconnected;
				AddressesToRemove.Add(TrackedAddress);
				Notifications.Add(ClientId);
			}
		}
	}

	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		for (const FMessageAddress& Address : AddressesToRemove)
		{
			for (auto It = AddressBook.CreateIterator(); It; ++It)
			{
				if (It->DiscoveryProtocolVersion < 2 && It->ControlEndpoint == Address)
				{
					// Maintain backwards compatibility, this used to be handled in the UI code.
					It.RemoveCurrent();
				}
			}
		}
	}

	for (const FLiveLinkHubClientId& Client : Notifications)
	{
		OnClientEventDelegate.Broadcast(Client, EClientEventType::Disconnected);
	}
}

void FLiveLinkHubProvider::ValidateHubConnections()
{
	ValidateConnections();

	const double MessageBusTimeout = GetDefault<ULiveLinkSettings>()->GetMessageBusTimeBeforeRemovingDeadSource();

	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		for (auto It = AddressBook.CreateIterator(); It; ++It)
		{
			if (!It->bConnected)
			{
				if (FPlatformTime::Seconds() - It->LastBeaconTimestamp > MessageBusTimeout)
				{
					if (It->DiscoveryProtocolVersion > 1)
					{
						// Previous versions of the discovery protocol did not use beacons so we can't rely on this.
						It.RemoveCurrent();
					}
				}
				else
				{
					if (!It->bDisableAutoconnect
						&& (!It->bSentDiscoveryRequest || FPlatformTime::Seconds() - It->LastDiscoveryRequestTimestamp > 1))
					{
						It->LastDiscoveryRequestTimestamp = FPlatformTime::Seconds();

						if (ShouldAutoConnectTo(It->Hostname, It->IP, It->ProjectName) 
							&& (GetDefault<ULiveLinkHubMessagingSettings>()->CanTransmitTo(FLiveLinkHub::Get()->GetTopologyMode(), It->TopologyMode))) // Previous versions of UE/LLH relied on the client side to check for Hub/Spoke compatibility.
						{
							It->bSentDiscoveryRequest = true;

							UE_LOGF(LogLiveLinkHub, Verbose, "Sending Hub Connection request to %ls", *It->Hostname);
							SendMessage(MakeLiveLinkHubDiscoveryMessage(), It->ControlEndpoint, EMessageFlags::Reliable, GetAnnotations());
						}
					}
				}
			}
		}
	}
}

FLiveLinkHubClientId FLiveLinkHubProvider::AddressToClientId(const FMessageAddress& Address) const
{
	FLiveLinkHubClientId ClientId;
	if (const FLiveLinkHubClientId* FoundId = AddressToIdCache.Find(Address))
	{
		ClientId = *FoundId;
	}
	else
	{
		UE_LOGF(LogLiveLinkHub, Warning, "Could not find a client for address %ls.", *Address.ToString());
	}
	
	return ClientId;
}

TArray<FLiveLinkHubClientId> FLiveLinkHubProvider::GetSessionClients() const
{
	TArray<FLiveLinkHubClientId> SessionClients;

	if (const TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
	{
		if (const TSharedPtr<ILiveLinkHubSession> CurrentSession = Manager->GetCurrentSession())
		{
			SessionClients = CurrentSession->GetSessionClients();
		}
	}

	return SessionClients;
}

TMap<FName, FString> FLiveLinkHubProvider::GetAnnotations() const
{
	TMap<FName, FString> AnnotationsCopy = Annotations;

	// Make a local copy of the annotations map and add the autoconnect annotation since this value might change over time.
	const FString AutoConnectValue = StaticEnum<ELiveLinkHubAutoConnectMode>()->GetNameStringByValue(static_cast<int64>(GetDefault<ULiveLinkHubUserSettings>()->GetAutoConnectMode()));
	AnnotationsCopy.Add(FLiveLinkHubMessageAnnotation::AutoConnectModeAnnotation, AutoConnectValue);

	// In case this is handled by the discovery manager (which doesn't directly handle LiveLinkHub messages)
	AnnotationsCopy.Add(FLiveLinkHubMessageAnnotation::IdAnnotation, FLiveLinkHub::Get()->GetId().ToString());

	// In case this is handled by the discovery manager (which doesn't directly handle LiveLinkHub messages)
	const ELiveLinkTopologyMode Mode = FLiveLinkHub::Get()->GetTopologyMode();
	FString TopopologyMode = StaticEnum<ELiveLinkTopologyMode>()->GetNameStringByValue(static_cast<int64>(Mode));
	AnnotationsCopy.Add(FLiveLinkMessageAnnotation::TopologyModeAnnotation, MoveTemp(TopopologyMode));

	TMap<FName, FString> BaseAnnotations = FLiveLinkProvider::GetAnnotations();
	BaseAnnotations.Append(AnnotationsCopy);

	return BaseAnnotations;
}

TArray<FLiveLinkHubClientId> FLiveLinkHubProvider::GetDiscoveredClients() const
{
	TArray<FLiveLinkHubClientId> DiscoveredClients;

	{
		auto CanTransmitTo = [](const FLiveLinkHubAddressBookEntry& Entry)
		{
			if (Entry.bConnected)
			{
				return false;
			}

			if (Entry.DiscoveryProtocolVersion > 1)
			{
				return GetDefault<ULiveLinkHubMessagingSettings>()->CanTransmitTo(FLiveLinkHub::Get()->GetTopologyMode(), Entry.TopologyMode);
			}
			else
			{
				// Previous version of the protocol was relying on the client to greenlight the connection.
				return true;
			}
		};

		UE::TReadScopeLock Lock(AddressBookLock);
		Algo::TransformIf(AddressBook, DiscoveredClients,
			CanTransmitTo,
			[](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ClientId; });
	}

	return DiscoveredClients;
}

TOptional<FLiveLinkHubDiscoveredClientInfo> FLiveLinkHubProvider::GetDiscoveredClientInfo(FLiveLinkHubClientId InClientId) const
{
	UE::TReadScopeLock Locker(AddressBookLock);

	TOptional<FLiveLinkHubDiscoveredClientInfo> ClientInfo;

	for (const FLiveLinkHubAddressBookEntry& Entry : AddressBook)
	{
		if (Entry.ClientId == InClientId)
		{
			ClientInfo = FLiveLinkHubDiscoveredClientInfo{};

			ClientInfo->Hostname = Entry.Hostname;
			ClientInfo->IP = Entry.IP;
			ClientInfo->LevelName = Entry.LevelName;
			ClientInfo->ProjectName = Entry.ProjectName;
			ClientInfo->TopologyMode = Entry.TopologyMode;

			break;
		}
	}

	return ClientInfo;
}

FText FLiveLinkHubProvider::GetClientDisplayName(FLiveLinkHubClientId InId) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	FText DisplayName;

	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(InId))
	{
		if (ClientInfoPtr->TopologyMode == ELiveLinkTopologyMode::Hub)
		{
			DisplayName = FText::FromString(ClientInfoPtr->LiveLinkInstanceName);
		}
		else
		{
			DisplayName = FText::FromString(FString::Format(TEXT("{0}"), { *ClientInfoPtr->Hostname }));
		}
	}
	else
	{
		UE::TReadScopeLock Lock(AddressBookLock);
		if (const FLiveLinkHubAddressBookEntry* Entry = AddressBook.FindByPredicate([InId](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ClientId == InId; }))
		{
			DisplayName = FText::FromString(FString::Format(TEXT("{0}"), { *Entry->Hostname }));
		}
		else
		{
			DisplayName = LOCTEXT("InvalidClientLabel", "Invalid Client");
		}
	}

	return DisplayName;
}

FText FLiveLinkHubProvider::GetClientStatus(FLiveLinkHubClientId Client) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
	{
		return StaticEnum<ELiveLinkClientStatus>()->GetDisplayNameTextByValue(static_cast<int64>(ClientInfoPtr->Status));
	}
		
	return LOCTEXT("InvalidStatus", "Disconnected");
}

bool FLiveLinkHubProvider::IsClientEnabled(FLiveLinkHubClientId Client) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
	{
		return ClientInfoPtr->bEnabled;
	}
	return false;
}

bool FLiveLinkHubProvider::IsClientConnected(FLiveLinkHubClientId Client) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
	{
		return ClientInfoPtr->Status == ELiveLinkClientStatus::Connected;
	}
	return false;
}

void FLiveLinkHubProvider::ConnectTo(FLiveLinkHubClientId Client)
{
	FLiveLinkHubAddressBookEntry Entry;

	{
		UE::TWriteScopeLock Lock(AddressBookLock);

		int32 EntryIndex = AddressBook.IndexOfByPredicate([Client](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ClientId == Client; });
		if (EntryIndex == INDEX_NONE)
		{
			return;
		}

		Entry = AddressBook[EntryIndex];

		if (Entry.DiscoveryProtocolVersion < 2)
		{
			// This allows adding/removing clients on previous versions.
			AddressBook[EntryIndex].bConnected = true;
		}
	}

	if (Entry.DiscoveryProtocolVersion < 2)
	{
		if (TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
		{
			if (TSharedPtr<ILiveLinkHubSession> CurrentSession = Manager->GetCurrentSession())
			{
				CurrentSession->AddClient(Client);
			}
		}
	}
	else
	{
		// Tag this discovery message as user-driven so the receiver bypasses its AutoConnect filter (which would otherwise refuse the connection when AutoConnect is Disabled).
		TMap<FName, FString> ExplicitAnnotations = GetAnnotations();
		ExplicitAnnotations.Add(FLiveLinkHubMessageAnnotation::ExplicitConnectAnnotation, TEXT("true"));
		SendMessage(MakeLiveLinkHubDiscoveryMessage(), Entry.ControlEndpoint, EMessageFlags::Reliable, MoveTemp(ExplicitAnnotations));
	}
}

void FLiveLinkHubProvider::SetClientEnabled(FLiveLinkHubClientId Client, bool bInEnable)
{
	{
		UE::TWriteScopeLock Locker(ClientsMapLock);
		if (FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
		{
			ClientInfoPtr->bEnabled = bInEnable;
		}
	}

	if (const TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
	{
		if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsTimecodeSource)
		{
			if (bInEnable)
			{
				// Enabling client, send it up to date timecode and custom time step settings.
				SendTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile(), Client);
			}
			else
			{
				// Disabling it, so reset its timecode and custom time step.
				ResetTimecodeSettings(Client);
			}
		}

		if (GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsCustomTimeStepSource)
		{
			if (bInEnable)
			{
				SendCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile(), Client);
			}
			else
			{
				ResetCustomTimeStepSettings(Client);
			}
		}

		if (bInEnable)
		{
			SendSourceEvaluationMode(GetDefault<ULiveLinkHubTimeAndSyncSettings>()->PreferredClientEvaluationMode, Client);
		}
	}
}

bool FLiveLinkHubProvider::IsSubjectEnabled(FLiveLinkHubClientId Client, FName SubjectName) const
{
	UE::TReadScopeLock Locker(ClientsMapLock);
	if (const FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
	{
		return !ClientInfoPtr->DisabledSubjects.Contains(SubjectName);
	}
	return false;
}

void FLiveLinkHubProvider::SetSubjectEnabled(FLiveLinkHubClientId Client, FName SubjectName, bool bInEnable)
{
	UE::TWriteScopeLock Locker(ClientsMapLock);
	if (FLiveLinkHubUEClientInfo* ClientInfoPtr = ClientsMap.Find(Client))
	{
		if (bInEnable)
		{
			ClientInfoPtr->DisabledSubjects.Remove(SubjectName);
		}
		else
		{
			ClientInfoPtr->DisabledSubjects.Add(SubjectName);
		}
	}
}

void FLiveLinkHubProvider::RequestAuxiliaryChannel(
	FLiveLinkHubClientId InClientId,
	FMessageEndpoint& InAuxEndpoint,
	UScriptStruct* InRequestTypeInfo,
	FLiveLinkHubAuxChannelRequestMessage& InRequest
)
{
	if (!InClientId.IsValid())
	{
		UE_LOGFMT(LogLiveLinkHub, Warning, "RequestAuxiliaryChannel: Invalid client ID.");
		return;
	}

	FMessageAddress RelevantControlEndpoint;

	{
		UE::TReadScopeLock Lock(AddressBookLock);
		for (const FLiveLinkHubAddressBookEntry& Entry : AddressBook)
		{
			if (Entry.bConnected && Entry.ClientId == InClientId)
			{
				RelevantControlEndpoint = Entry.ControlEndpoint;
				break;
			}
		}
	}

	if (RelevantControlEndpoint.IsValid())
	{
		InAuxEndpoint.Send(&InRequest, InRequestTypeInfo, EMessageFlags::Reliable, {}, nullptr, { RelevantControlEndpoint }, FTimespan::Zero(), FDateTime::MaxValue());
	}
	else
	{
		UE_LOGFMT(LogLiveLinkHub, Warning, "Could not find address for client {ClientId}.", InClientId.ToString());
	}
}

FLiveLinkHubClientId FLiveLinkHubProvider::UpdateAddressBookEntry(const FMessageAddress& ControlEndpointAddress, int32 DiscoveryProtocolVersion, ELiveLinkTopologyMode TopologyMode, const FString& Hostname, const FString& ProjectName, const FString& LevelName)
{
	TOptional<FLiveLinkHubAddressBookEntry> ReadOnlyEntry;

	FLiveLinkHubClientId ClientId;

	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		
		if (FLiveLinkHubAddressBookEntry* Entry = AddressBook.FindByPredicate([Address = ControlEndpointAddress](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ControlEndpoint == Address; }))
		{
			ClientId = Entry->ClientId;
			Entry->LastBeaconTimestamp = FPlatformTime::Seconds();
			Entry->TopologyMode = TopologyMode; // This may have changed since the last beacon.
			// Maybe add current level as well ?

			if (Entry->IP.IsEmpty())
			{
				FString IP = LiveLinkHubProviderUtils::GetIPAddress(ControlEndpointAddress);
				if (!IP.IsEmpty())
				{
					Entry->IP = MoveTemp(IP);
				}
			}

			if (!ProjectName.IsEmpty() && Entry->ProjectName != ProjectName)
			{
				Entry->ProjectName = ProjectName;
			}

			if (!LevelName.IsEmpty() && Entry->LevelName != LevelName)
			{
				Entry->LevelName = LevelName;
			}

			ReadOnlyEntry = *Entry;
		}
	}

	if (ReadOnlyEntry)
	{
		bool bModifiedClientsMap = false;
		{
			UE::TWriteScopeLock Lock(ClientsMapLock);
			// Update the clients info from the address book entry.
			// Todo: Ideally, this map and the address book should be one and the same.
			if (FLiveLinkHubUEClientInfo* Info = ClientsMap.Find(ReadOnlyEntry->ClientId); Info && Info->IPAddress != ReadOnlyEntry->IP)
			{
				if (Info->IPAddress != ReadOnlyEntry->IP)
				{
					Info->IPAddress = ReadOnlyEntry->IP;
					bModifiedClientsMap = true;
				}
				if (Info->ProjectName != ReadOnlyEntry->ProjectName && !ReadOnlyEntry->ProjectName.IsEmpty())
				{
					Info->ProjectName = ReadOnlyEntry->ProjectName;
					bModifiedClientsMap = true;
				}
			}
		}
	
		if (bModifiedClientsMap)
		{
			// todo run this code again if auto connect behavior changes.
			OnClientEventDelegate.Broadcast(ReadOnlyEntry->ClientId, EClientEventType::Modified);
		}
	}
	else
	{
		FLiveLinkHubAddressBookEntry NewEntry =
		{
			.ControlEndpoint = ControlEndpointAddress,
			.DataEndpoint = DiscoveryProtocolVersion < 2 ? ControlEndpointAddress : FMessageAddress{},
			.SourceGuid = FGuid{},
			.TopologyMode = TopologyMode,
			.IP = LiveLinkHubProviderUtils::GetIPAddress(ControlEndpointAddress),
			.Hostname = Hostname,
			.ProjectName = ProjectName,
			.LevelName = LevelName,
			.LastBeaconTimestamp = FPlatformTime::Seconds(),
			.DiscoveryProtocolVersion = DiscoveryProtocolVersion
		};

		{
			UE::TWriteScopeLock Lock(AddressBookLock);
			AddressBook.Add(NewEntry);
		}

		ClientId = NewEntry.ClientId;

		if (DiscoveryProtocolVersion > 1)
		{
			// todo run this code again if auto connect behavior changes.
			OnClientEventDelegate.Broadcast(NewEntry.ClientId, EClientEventType::Discovered);
		}
	}

	return ClientId;
}

FLiveLinkHubDiscoveryMessage* FLiveLinkHubProvider::MakeLiveLinkHubDiscoveryMessage() const
{
	return FMessageEndpoint::MakeMessage<FLiveLinkHubDiscoveryMessage>(GetProviderName(), FLiveLinkHub::Get()->GetTopologyMode(), FLiveLinkHub::Get()->GetId());
}

void FLiveLinkHubProvider::HandleBeaconMessage(const FLiveLinkHubBeaconMessage& Message, const TSharedRef<class IMessageContext, ESPMode::ThreadSafe>& Context)
{
	// Only consider beacon messages that don't come from this process.
	if (Message.InstanceId != FApp::GetInstanceId())
	{
		// Beacons were added in the second version of the discovery protocol.
		constexpr int32 DiscoveryProtocolVersion = 2;
		UpdateAddressBookEntry(Context->GetSender(), DiscoveryProtocolVersion, Message.TopologyMode, Message.Hostname, Message.ProjectName, Message.LevelName);
	}
}

bool FLiveLinkHubProvider::ShouldAutoConnectTo(const FString& RemoteHostname, const FString& IP, const FString& ProjectName) const
{
	const ELiveLinkHubAutoConnectMode AutoConnectClients = GetDefault<ULiveLinkHubUserSettings>()->GetAutoConnectMode();
	if (AutoConnectClients == ELiveLinkHubAutoConnectMode::Disabled)
	{
		return false;
	}

	if (AutoConnectClients == ELiveLinkHubAutoConnectMode::LocalOnly)
	{
		// LocalOnly does not apply filters because it's considered an override.
		return RemoteHostname == GetMachineName();
	}

	FLiveLinkHubClientFilterPreset Preset = GetMutableDefault<ULiveLinkHubUserSettings>()->GetClientFiltersData_AnyThread();

	bool bIncluded = false;
	bool bHasAtLeastOneIncludeFilter = false;

	for (const FLiveLinkHubClientTextFilter& Filter : Preset.Filters)
	{
		bHasAtLeastOneIncludeFilter |= Filter.Behavior == ELiveLinkHubClientFilterBehavior::Include && (!Filter.Text.IsEmpty() || !Filter.Project.IsEmpty());

		const FString* StringToCompare = Filter.Type == ELiveLinkHubClientFilterType::IP ? &IP : &RemoteHostname;
		// Allow filtering on project name if IP/Hostname is empty.
		if (StringToCompare->MatchesWildcard(Filter.Text) || (Filter.Text.IsEmpty() && !Filter.Project.IsEmpty()))
		{
			const bool MatchesProject = Filter.Project.IsEmpty() || Filter.Project == TEXT("*") || ProjectName.MatchesWildcard(Filter.Project);
			
			if (MatchesProject)
			{
				if (Filter.Behavior == ELiveLinkHubClientFilterBehavior::Include)
				{
					// We don't return true here because it might get excluded in a subsequent filter.
					bIncluded = true;
				}
				else
				{
					// Note: Exclusion currently has priority over inclusion filters.
					return false;
				}
			}
		}
	}


	if (bIncluded)
	{
		return true;
	}

	// If there's at least one filter and it wasn't matched, then we discard it.
	return !bHasAtLeastOneIncludeFilter;
}

void FLiveLinkHubProvider::OnMessageBusNotification(const FMessageBusNotification& Notification)
{
	if (Notification.NotificationType == EMessageBusNotification::Unregistered)
	{
		TArray<FLiveLinkHubClientId> RemovedClients;

		UE::TWriteScopeLock Lock(AddressBookLock);

		for (auto EntryIt = AddressBook.CreateIterator(); EntryIt; ++EntryIt)
		{
			if (EntryIt->ControlEndpoint == Notification.RegistrationAddress)
			{
				RemovedClients.Add(EntryIt->ClientId);
				EntryIt.RemoveCurrent();
			}
		}

		for (const FLiveLinkHubClientId& ClientId : RemovedClients)
		{
			OnClientEventDelegate.Broadcast(ClientId, EClientEventType::Disconnected);
		}
	}
}

void FLiveLinkHubProvider::OnFiltersModified()
{
	TSet<FLiveLinkHubClientId> ClientsToConnect;
	TSet<FLiveLinkHubClientId> ClientsToDisconnect;

	{
		UE::TWriteScopeLock Lock(AddressBookLock);
		for (auto EntryIt = AddressBook.CreateIterator(); EntryIt; ++EntryIt)
		{
			const bool bShouldConnectResult = ShouldAutoConnectTo(EntryIt->Hostname, EntryIt->IP, EntryIt->ProjectName)
				&& GetDefault<ULiveLinkHubMessagingSettings>()->CanTransmitTo(FLiveLinkHub::Get()->GetTopologyMode(), EntryIt->TopologyMode);

			if (EntryIt->bConnected)
			{
				if (!bShouldConnectResult)
				{
					ClientsToDisconnect.Add(EntryIt->ClientId);
				}
			}
			else if (bShouldConnectResult)
			{
				ClientsToConnect.Add(EntryIt->ClientId);
			}
		}


		for (auto EntryIt = AddressBook.CreateIterator(); EntryIt; ++EntryIt)
		{
			if (ClientsToConnect.Contains(EntryIt->ClientId))
			{
				EntryIt->bSentDiscoveryRequest = true;
				EntryIt->bDisableAutoconnect = false;

				UE_LOGF(LogLiveLinkHub, Verbose, "Sending Hub Connection request to %ls", *EntryIt->Hostname);
				SendMessage(MakeLiveLinkHubDiscoveryMessage(), EntryIt->ControlEndpoint, EMessageFlags::Reliable, GetAnnotations());
			}
		}
	}

	if (TSharedPtr<ILiveLinkHubSessionManager> Manager = SessionManager.Pin())
	{
		for (const FLiveLinkHubClientId& ClientId : ClientsToDisconnect)
		{
			Manager->GetCurrentSession()->RemoveClient(ClientId);
		}
	}
}

#undef LOCTEXT_NAMESPACE /*LiveLinkHub.LiveLinkHubProvider*/
