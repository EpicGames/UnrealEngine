// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Clients/GameThreadMessageHandler.h"
#include "Clients/LiveLinkHubUEClientInfo.h"
#include "Engine/TimerHandle.h"
#include "ILiveLinkHubClientsModel.h"
#include "IMessageContext.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkProviderImpl.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/Function.h"


class ILiveLinkHubSessionManager;
struct FLiveLinkHubClientId;
struct FLiveLinkHubConnectMessage;
struct FLiveLinkHubUEClientInfo;

struct FLiveLinkHubAddressBookEntry
{
	/** Address of the MessagingModule endpoint that handles 'control' messages like Disconnect, Timecode and Genlock. */
	FMessageAddress ControlEndpoint;
	/** 
	 * Address of the actual LiveLink source that handles LiveLink data. 
	 * @note: Only set after we receive a connection message.
	 */
	FMessageAddress DataEndpoint;
	/** Id of the message bus source on the client side. */
	FGuid SourceGuid;
	/** TopologyMode of this entry. */
	ELiveLinkTopologyMode TopologyMode = ELiveLinkTopologyMode::UnrealClient;
	/** IP of the client. */
	FString IP;
	/** Hostname of the client. */
	FString Hostname;
	/** Project that the client is in. */
	FString ProjectName;
	/** Name of the current level. */
	FString LevelName;
	/** Timestamp of the last beacon message received for this client. */
	double LastBeaconTimestamp = 0.0;
	/** This is set when a user removes a client from the session so as to not autoconnect it again. */
	bool bDisableAutoconnect = false;
	// Whether the provider has connected to it.
	bool bConnected = false;
	/** Whether discovery (connection request) has been sent. */
	bool bSentDiscoveryRequest = false;
	/** Timestamp of the last discovery request messsage that was sent out. */
	double LastDiscoveryRequestTimestamp = 0.0;
	/** ID of this client. */
	FLiveLinkHubClientId ClientId = FLiveLinkHubClientId::NewId();
	/** Version of the discovery protocol. V1 = Before 5.7, V2 = 5.7, after bringing the MBus source control logic to the LLH Messaging module. */
	int32 DiscoveryProtocolVersion = 1;
};

/** 
 * LiveLink Provider that allows getting more information about a UE client by communicating with a LiveLinkHub MessageBus Source.
 */
class FLiveLinkHubProvider : public FLiveLinkProvider, public ILiveLinkHubClientsModel, public TSharedFromThis<FLiveLinkHubProvider>
{
public:
	using FLiveLinkProvider::SendClearSubjectToConnections;
	using FLiveLinkProvider::GetLastSubjectStaticDataStruct;
	using FLiveLinkProvider::GetProviderName;

	/**
	 * Create a message bus handler that will dispatch messages on the game thread. 
	 * This is useful to receive some messages on AnyThread and delegate others on the game thread (ie. for methods that will trigger UI updates which need to happen on game thread. )
	 */
	template <typename MessageType>
	TSharedRef<TGameThreadMessageHandler<MessageType, FLiveLinkHubProvider>> MakeHandler(typename TGameThreadMessageHandler<MessageType, FLiveLinkHubProvider>::FuncType Func)
	{
		return MakeShared<TGameThreadMessageHandler<MessageType, FLiveLinkHubProvider>>(this, Func);
	}

	FLiveLinkHubProvider(const TSharedRef<ILiveLinkHubSessionManager>& InSessionManager, const FString& ProviderName);

	virtual ~FLiveLinkHubProvider() override;

	//~ Begin LiveLinkProvider interface
	virtual bool ShouldTransmitToSubject_AnyThread(FName SubjectName, FMessageAddress Address) const override;
	virtual TOptional<FLiveLinkHubUEClientInfo> GetClientInfo(FLiveLinkHubClientId InClient) const override;
	virtual void OnMessageBusNotification(const FMessageBusNotification& Notification) override;
	//~ End LiveLinkProvider interface

	/**
	 * Restore a client, calling this will modify the client ID if it matches an existing connection.
	 */
	void AddRestoredClient(FLiveLinkHubUEClientInfo& InOutRestoredClientInfo);

	/** Retrieve the existing client map. */
	const TMap<FLiveLinkHubClientId, FLiveLinkHubUEClientInfo>& GetClientsMap() const { return ClientsMap; }

	/** 
	 * Timecode settings that should be shared to connected editors. 
	 * @Note If ClientId is not provided or invalid, the message will be broadcast to all connected clients.
	 */
	void UpdateTimecodeSettings(const FLiveLinkHubTimecodeSettings& InSettings, const FLiveLinkHubClientId& ClientId = FLiveLinkHubClientId{});

	/**
	 * Reset timecode settings for all connected clients. 
	 * @Note If ClientId is not provided or invalid, the message will be broadcast to all connected clients.
	 */
	void ResetTimecodeSettings(const FLiveLinkHubClientId& ClientId = FLiveLinkHubClientId{});

	/** Frame Lock settings that should be shared to connected editors. */
	void UpdateCustomTimeStepSettings(const FLiveLinkHubCustomTimeStepSettings& InSettings, const FLiveLinkHubClientId& ClientId = FLiveLinkHubClientId{});

	/**
	 * Reset Frame Lock settings on connected editors.
	 * @Note If ClientId is not provided or invalid, the message will be broadcast to all connected clients.
	 */
	void ResetCustomTimeStepSettings(const FLiveLinkHubClientId& ClientId = FLiveLinkHubClientId{});

	/**
	 * Push the hub's preferred evaluation mode to connected Editor clients, overwriting their hub source's local Mode.
	 * @Note If ClientId is not provided or invalid, the message will be broadcast to all connected clients.
	 */
	void UpdateSourceEvaluationMode(ELiveLinkSourceMode Mode, const FLiveLinkHubClientId& ClientId = FLiveLinkHubClientId{});

	/** Connect to all instances in the address book that are compatible with the host's topology mode. */
	void ConnectToAllDiscoveredClients();

	/**
	 * Send a disconnect message to all connected clients.
	 */
	void DisconnectAll();

	/**
	 * Send a disconnect message to a single client.
	 */
	void DisconnectClient(const FLiveLinkHubClientId& ClientId);

private:
	/** Handle a connection message resulting from a livelink hub message bus source connecting to this provider. */
	void HandleHubConnectMessage(const FLiveLinkHubConnectMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	
	/** Handle a client info message being received. Happens when new information about a client is received (ie. Client has changed map) */
	void HandleClientInfoMessage(const FLiveLinkClientInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handle a client (UE or Hub) sending a disconnect request. */
	void HandleHubDisconnectMessage(const FLiveLinkHubDisconnectMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Send timecode settings to connected UE Clients. */
	void SendTimecodeSettings(const FLiveLinkHubTimecodeSettings& InSettings, const FLiveLinkHubClientId& ClientId);

	/** Send CustomTimeStep settings to connected UE Clients. */
	void SendCustomTimeStepSettings(const FLiveLinkHubCustomTimeStepSettings& InSettings, const FLiveLinkHubClientId& ClientId);

	/** Send a source evaluation mode override to connected UE Clients. */
	void SendSourceEvaluationMode(ELiveLinkSourceMode Mode, const FLiveLinkHubClientId& ClientId);

	/** Send a message to clients that are connected and enabled through the hub clients list. */
	template<typename MessageType>
	void SendControlMessageToEnabledClients(MessageType* Message, EMessageFlags Flags = EMessageFlags::None)
	{
		TArray<FMessageAddress> ValidClientAddresses;

		{
			UE::TReadScopeLock Lock(AddressBookLock);
			Algo::TransformIf(AddressBook, ValidClientAddresses,
				[this](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.bConnected && ShouldTransmitToClient_AnyThread(Entry.ClientId); },
				[](const FLiveLinkHubAddressBookEntry& Entry) { return Entry.ControlEndpoint; });
		}

		SendMessage(Message, ValidClientAddresses, Flags);
	}

	/**
	 * Whether a message should be transmitted to a particular client.
	 * You can specify a SubjectName if you want to know if the session allows transmitting to that subject as well.
	 **/
	bool ShouldTransmitToClient_AnyThread(const FLiveLinkHubClientId& ClientId, FLiveLinkSubjectName SubjectName = FLiveLinkSubjectName{}) const;

	void CloseConnections(const TArray<FMessageAddress>& ClosedAddresses);

	/** Validate connections and entries in the address book. */
	void ValidateHubConnections();

	/** 
	 * Get the client id that corresponds to this address from our cache. 
	 * May return an invalid ID if the address is not in the cache (ie. If client is disconnecting))
	 **/
	FLiveLinkHubClientId AddressToClientId(const FMessageAddress& Address) const;

	/** Disconnect client based on its entry in the address book. */
	void DisconnectClient(const FLiveLinkHubAddressBookEntry& EntryToDisconnect);

	/** Handle beacon message to add this entry to our discovered clients. Beacons are sent after a pong message with the discovery request annotation is sent. */
	void HandleBeaconMessage(const FLiveLinkHubBeaconMessage& Message, const TSharedRef<class IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Returns whether we should automatically connect to this client based on our settings. */
	bool ShouldAutoConnectTo(const FString& RemoteHostname, const FString& IP = FString(), const FString& ProjectName = FString()) const;

	/** 
	 * Update (or creates if needed) an entry for this address.
	 * @return the client ID.
	 */
	FLiveLinkHubClientId UpdateAddressBookEntry(const FMessageAddress& ControlEndpointAddress, int32 DiscoveryProtocolVersion, ELiveLinkTopologyMode TopologyMode = ELiveLinkTopologyMode::UnrealClient, const FString& Hostname = FString(), const FString& ProjectName = FString(), const FString& LevelName = FString());

	/** Utility method that constructs a discovery message. */
	FLiveLinkHubDiscoveryMessage* MakeLiveLinkHubDiscoveryMessage() const;

	/** Invoked when the user modifies the auto-connect filtering options. */
	void OnFiltersModified();

	/**
	 * Connect to a set of clients.
	 * @Note The clients must have been discovered before invoking this for the connection message to be sent.
	 */
	void ConnectToClients(const TArray<FLiveLinkHubClientId>& DiscoveredClientIds);

protected:
	//~ Begin ILiveLinkHubClientsModel interface
	virtual void OnConnectionsClosed(const TArray<FMessageAddress>& ClosedAddresses) override;
	virtual TArray<FLiveLinkHubClientId> GetSessionClients() const override;
	virtual TMap<FName, FString> GetAnnotations() const override;
	virtual TArray<FLiveLinkHubClientId> GetDiscoveredClients() const override;
	virtual TOptional<FLiveLinkHubDiscoveredClientInfo> GetDiscoveredClientInfo(FLiveLinkHubClientId InClientId) const override;
	virtual FText GetClientDisplayName(FLiveLinkHubClientId InAddress) const override;
	virtual FOnClientEvent& OnClientEvent() override
	{
		return OnClientEventDelegate;
	}
	virtual FText GetClientStatus(FLiveLinkHubClientId Client) const override;
	virtual bool IsClientEnabled(FLiveLinkHubClientId Client) const override;
	virtual bool IsClientConnected(FLiveLinkHubClientId Client) const override;
	virtual void ConnectTo(FLiveLinkHubClientId Client) override;
	virtual void SetClientEnabled(FLiveLinkHubClientId Client, bool bInEnable) override;
	virtual bool IsSubjectEnabled(FLiveLinkHubClientId Client, FName SubjectName) const override;
	virtual void SetSubjectEnabled(FLiveLinkHubClientId Client, FName SubjectName, bool bInEnable) override;
	virtual void RequestAuxiliaryChannel(FLiveLinkHubClientId InClientId, FMessageEndpoint& InAuxEndpoint, UScriptStruct* InRequestTypeInfo, FLiveLinkHubAuxChannelRequestMessage& InRequest) override;
	//~ End ILiveLinkHubClientsModel interface


private:
	/** Holds the info and state of all clients have been discovered. */
	TArray<FLiveLinkHubAddressBookEntry> AddressBook;
	/** Handle to the timer responsible for validating the livelinkprovider's connections.*/
	FTimerHandle ValidateConnectionsTimer;
	/** List of information we have on clients we have discovered. */
	TMap<FLiveLinkHubClientId, FLiveLinkHubUEClientInfo> ClientsMap;
	/** Delegate called when the provider receives a client change. */
	FOnClientEvent OnClientEventDelegate;
	/** Annotations sent with every message from this provider. In our case it's use to disambiguate a livelink hub provider from other livelink providers.*/
	TMap<FName, FString> Annotations;
	/** LiveLinkHub session manager. */
	TWeakPtr<ILiveLinkHubSessionManager> SessionManager;
	/** Cache used to retrieve the client id from a message bus address. */
	TMap<FMessageAddress, FLiveLinkHubClientId> AddressToIdCache;
	/** Keeps track of a client being disconnected to avoid sending back a message to disconnect it. */
	TOptional<FLiveLinkHubClientId> ClientBeingDisconnected;
	/** Lock used to access the clients map from different threads. */
	mutable FTransactionallySafeRWLock ClientsMapLock;
	/** Lock used to access the AddressBook from different threads. */
	mutable FTransactionallySafeRWLock AddressBookLock;

};
