// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Settings/LiveLinkHubSettings.h"

class SLiveLinkHubClientFilters : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkHubClientFilters) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			CreatePropertyEditor()
		];
	}

	/** Create a property editor that displays the filters taken from ULiveLinkHubUserSettings. */
	TSharedRef<SWidget> CreatePropertyEditor()
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = NAME_None;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.ColumnWidth = 0.8f;

		TSharedRef<IDetailsView> DetailView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
		DetailView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SLiveLinkHubClientFilters::ShouldDisplayProperty));
		DetailView->SetObject(GetMutableDefault<ULiveLinkHubUserSettings>());
		DetailView->SetRootExpansionStates(true, true);

		return DetailView;
	}

	/** Hides properties that aren't in the filters category. */
	bool ShouldDisplayProperty(const FPropertyAndParent& InPropertyAndParent) const
	{
		static const FLazyName CategoryName = TEXT("Category");
		static const FString FiltersCategory = TEXT("Filters");
		if (InPropertyAndParent.Property.GetMetaData(CategoryName) == FiltersCategory)
		{
			return true;
		}
		if (!InPropertyAndParent.ParentProperties.IsEmpty())
		{
			if (InPropertyAndParent.ParentProperties.Last()->GetMetaData(CategoryName) == FiltersCategory)
			{
				return true;
			}
		}
		return false;
	}
};