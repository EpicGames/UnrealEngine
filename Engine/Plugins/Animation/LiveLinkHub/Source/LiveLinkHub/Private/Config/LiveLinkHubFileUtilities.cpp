// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubFileUtilities.h"

#include "Config/LiveLinkHubTemplateTokens.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubLog.h"
#include "Misc/FileHelper.h"
#include "Session/LiveLinkHubSessionData.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "UObject/Package.h"

void UE::LiveLinkHub::FileUtilities::Private::SaveConfig(const ULiveLinkHubSessionData* InConfigData, const FString& InFilePath)
{
	if (ensure(!InFilePath.IsEmpty()))
	{
		FString JsonString;

		const TSharedPtr<FJsonObject> JsonObject = ToJson(InConfigData);
		const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);

		FFileHelper::SaveStringToFile(JsonString, *InFilePath, FFileHelper::EEncodingOptions::ForceUTF8);
	}
}

ULiveLinkHubSessionData* UE::LiveLinkHub::FileUtilities::Private::LoadConfig(const FString& InFilePath)
{
	if (IFileManager::Get().FileExists(*InFilePath))
	{
		FString JsonConfig;
		if (!FFileHelper::LoadFileToString(JsonConfig, *InFilePath))
		{
			return nullptr;
		}

		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonConfig);

		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		bool bDeserializeResult = FJsonSerializer::Deserialize(JsonReader, JsonObject);
		if (!bDeserializeResult || !JsonObject.IsValid())
		{
			// This may have failed because we're reading a file that was saved before encoding was saved explicitly in the file.
			// So revert to the old read path that uses an explicit FileReader. 
			if (TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InFilePath)))
			{
				const TSharedRef<TJsonReader<>> FileReaderJsonReader = TJsonReaderFactory<>::Create(Ar.Get());
				bDeserializeResult = FJsonSerializer::Deserialize(FileReaderJsonReader, JsonObject) && JsonObject.IsValid();
			}
		}

		if (bDeserializeResult)
		{
			int32 Version = 0;
			if (JsonObject->TryGetNumberField(JsonVersionKey, Version) &&
				Version <= LiveLinkHubVersion)
			{
				return FromJson(JsonObject);
			}

			UE_LOGF(LogLiveLinkHub, Error, "Could not load config %ls because the version field %ls was missing or invalid.",
				*InFilePath, *JsonVersionKey);
		}
	}

	return nullptr;
}

TSharedPtr<FJsonObject> UE::LiveLinkHub::FileUtilities::Private::ToJson(const ULiveLinkHubSessionData* InConfigData)
{
	TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	const TSharedPtr<FJsonValueNumber> NumberValue = MakeShared<FJsonValueNumber>(LiveLinkHubVersion);
	JsonObject->SetField(JsonVersionKey, NumberValue);
	
	FJsonObjectConverter::UStructToJsonObject(ULiveLinkHubSessionData::StaticClass(), InConfigData, JsonObject);
	
	return JsonObject;
}

ULiveLinkHubSessionData* UE::LiveLinkHub::FileUtilities::Private::FromJson(const TSharedPtr<FJsonObject>& InJsonObject)
{
	if (InJsonObject.IsValid())
	{
		ULiveLinkHubSessionData* OutConfigData = NewObject<ULiveLinkHubSessionData>(GetTransientPackage());
		const bool bResult =
			FJsonObjectConverter::JsonObjectToUStruct(InJsonObject.ToSharedRef(),
				ULiveLinkHubSessionData::StaticClass(), OutConfigData);

		if (!bResult)
		{
			UE_LOGF(LogLiveLinkHub, Error, "Could not convert from json to LiveLinkHubSessionData.")
		}

		return OutConfigData;
	}

	return nullptr;
}
