// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class ULiveLinkHubSessionData;
class FJsonObject;

namespace UE::LiveLinkHub::FileUtilities::Private
{
	/** A key which must be present in json configs and mapped to the saved json version. */
	const FString JsonVersionKey = TEXT("liveLinkHub");
	/** The json version we support. */
	constexpr int32 LiveLinkHubVersion = 1;

	/** The extension of the config file. */
	const FString ConfigExtension = TEXT("json");
	/** The default name of the config file. */
	const FString ConfigDefaultFileName = TEXT("LiveLinkHubConfig");
	/** The description of the config file. */
	const FString ConfigDescription = TEXT("Live Link Hub Config");

	/** Save config data to disk. */
	void SaveConfig(const ULiveLinkHubSessionData* InConfigData, const FString& InFilePath);

	/** Load config data from disk. */
	ULiveLinkHubSessionData* LoadConfig(const FString& InFilePath);

	/** Convert config data to json. */
	TSharedPtr<FJsonObject> ToJson(const ULiveLinkHubSessionData* InConfigData);

	/** Convert config data from json. */
	ULiveLinkHubSessionData* FromJson(const TSharedPtr<FJsonObject>& InJsonObject);

	/** Evaluated results from a template file string. */
	struct FFilenameTemplateData
	{
		/** Full original path with evaluated tokens. */
		FString FullPath;
		/** The evaluated folder(s) without the first forward slash or filename. */
		FString FolderPath;
		/** The evaluated filename. */
		FString FileName;
	};
}
