// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubTemplateTokens.h"

#include "ILiveLinkHubModule.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "Session/LiveLinkHubSessionManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkHubTemplateTokens)

#define LOCTEXT_NAMESPACE "LiveLinkHubNamingTokens"

ULiveLinkHubNamingTokens::ULiveLinkHubNamingTokens()
{
	Namespace = ILiveLinkHubModule::GetLiveLinkHubNamingTokensNamespace();
	NamespaceDisplayName = LOCTEXT("NamespaceDisplayName", "Live Link Hub");
}

void ULiveLinkHubNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
	Super::OnCreateDefaultTokens(Tokens);

	Tokens.Add({
		TEXT("session"),
		LOCTEXT("SessionName", "Session Name from the Session Entry Widget"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
			return FText::FromString(FString::Printf(TEXT("%s"), *SessionName));
		})
	});

	Tokens.Add({
		TEXT("slate"),
		LOCTEXT("SlateName", "Slate Name from the Session Entry Widget"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
			return FText::FromString(FString::Printf(TEXT("%s"), *SlateName));
		})
	});

	Tokens.Add({
		TEXT("take"),
		LOCTEXT("TakeNumber", "Take Number from the Session Entry Widget"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
			FNumberFormattingOptions NumberFormatOptions;
			NumberFormatOptions.MinimumIntegralDigits = 2;

			return FText::AsNumber(TakeNumber, &NumberFormatOptions);
		})
	});
	
	Tokens.Add({
		TEXT("config"),
		LOCTEXT("ConfigName", "Loaded Live Link Hub Config Name"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
			return FText::FromString(FString::Printf(TEXT("%s"), *ConfigName));
		})
	});
}

void ULiveLinkHubNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
	if (const FLiveLinkHubModule* LiveLinkHubModule = FModuleManager::Get().GetModulePtr<FLiveLinkHubModule>("LiveLinkHub"))
	{
		if (const TSharedPtr<FLiveLinkHub> LiveLinkHub = LiveLinkHubModule->GetLiveLinkHub())
		{
			if (const TSharedPtr<ILiveLinkHubSessionManager> SessionManager = LiveLinkHub->GetSessionManager())
			{
				ConfigName = FPaths::GetBaseFilename(SessionManager->GetLastConfigPath());
				
				const ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();
				SessionName = SessionInfo.GetSessionName();
				TakeNumber = SessionInfo.GetTakeNumber();
				SlateName = SessionInfo.GetSlateName();
			}
		}
	}
}

void ULiveLinkHubNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
	ConfigName.Empty();
	SessionName.Empty();
	SlateName.Empty();
	TakeNumber = 0;
}

#undef LOCTEXT_NAMESPACE
