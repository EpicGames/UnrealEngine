// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "LiveLinkHubTemplateTokens.generated.h"

UCLASS(NotBlueprintable)
class ULiveLinkHubNamingTokens final: public UNamingTokens
{
	GENERATED_BODY()

public:
	ULiveLinkHubNamingTokens();

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens
	
private:
	/** The loaded config name. */
	FString ConfigName;
	
	/** The current session. */
	FString SessionName;

	/** Session slate name */
	FString SlateName;
	
	/** Session take number. */
	int32 TakeNumber = 0;
};
