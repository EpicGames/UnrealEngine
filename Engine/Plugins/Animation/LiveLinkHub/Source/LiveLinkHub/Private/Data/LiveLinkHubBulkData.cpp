// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubBulkData.h"

#include "LiveLinkHubLog.h"

#include "Serialization/LargeMemoryReader.h"

FLiveLinkHubBulkData::FScopedBulkDataMemoryReader::FScopedBulkDataMemoryReader(const int64 InOffset, const int64 InBytesToRead, FLiveLinkHubBulkData* InBulkData)
{
	Memory.SetNumUninitialized(InBytesToRead);

	LocalBulkDataOffset = InBulkData->ReadBulkDataImpl(InOffset, InBytesToRead, Memory.GetData());
	MemoryReader = MakeUnique<FLargeMemoryReader>(Memory.GetData(), InBytesToRead);
}

FLiveLinkHubBulkData::~FLiveLinkHubBulkData()
{
	UnloadBulkData();
}

void FLiveLinkHubBulkData::CloseFileReader()
{
	RecordingFileReader.Reset();
}

void FLiveLinkHubBulkData::UnloadBulkData()
{
	CloseFileReader();
	BulkData.UnloadBulkData();
}

void FLiveLinkHubBulkData::ReadBulkData(const int64 InBytesToRead, uint8* InMemory)
{
	BulkDataOffset = ReadBulkDataImpl(BulkDataOffset, InBytesToRead, InMemory);
}

TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> FLiveLinkHubBulkData::CreateBulkDataMemoryReader(const int64 InBytesToRead)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLiveLinkHubBulkData::CreateBulkDataMemoryReader"), STAT_FLiveLinkHubBulkData_CreateBulkDataMemoryReader, STATGROUP_LiveLinkHub);
	
	TSharedPtr<FScopedBulkDataMemoryReader> Reader = MakeShared<FScopedBulkDataMemoryReader>(BulkDataOffset, InBytesToRead, this);
	BulkDataOffset = Reader->GetBulkDataOffset();

	return Reader;
}

void FLiveLinkHubBulkData::ResetBulkDataOffset()
{
	BulkDataOffset = BulkData.GetBulkDataOffsetInFile();
}

void FLiveLinkHubBulkData::SetBulkDataOffset(const int64 InNewOffset)
{
	BulkDataOffset = InNewOffset;
}

void FLiveLinkHubBulkData::Serialize(FArchive& Ar, UObject* Owner)
{
	BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
	BulkData.Serialize(Ar, Owner);
}

void FLiveLinkHubBulkData::WriteBulkData(TArray64<uint8>& Data)
{
	BulkData.Lock(LOCK_READ_WRITE);
	BulkData.Realloc(Data.Num());
	unsigned char* BulkDataPtr = BulkData.Realloc(Data.Num());
	FMemory::Memmove(BulkDataPtr, Data.GetData(), Data.Num());
	BulkData.Unlock();
}

int64 FLiveLinkHubBulkData::ReadBulkDataImpl(int64 InOffset, int64 InBytesToRead, uint8* InMemory)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLiveLinkHubBulkData::ReadBulkDataImpl"), STAT_FLiveLinkHubBulkData_ReadBulkDataImpl, STATGROUP_LiveLinkHub);
	
	if (!RecordingFileReader.IsValid())
	{
		check(BulkData.DoesExist());
		check(BulkData.CanLoadFromDisk());
		check(!BulkData.IsInlined());
		check(!BulkData.IsInSeparateFile());
		check(!BulkData.IsBulkDataLoaded());

		RecordingFileReader = TUniquePtr<IAsyncReadFileHandle>(BulkData.OpenAsyncReadHandle());
	}

	check(RecordingFileReader);

	const TUniquePtr<IAsyncReadRequest> ReadRequest(RecordingFileReader->ReadRequest(
		InOffset,
		InBytesToRead,
		AIOP_High,
		nullptr,
		InMemory
		));
	ReadRequest->WaitCompletion();

	return InOffset + InBytesToRead;
}
