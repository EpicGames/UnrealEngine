// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/BulkData.h"

class FBufferArchive;
class FLargeMemoryReader;
class IAsyncReadFileHandle;

/** Bulk data for live link hub, automatically tracking the current offset of the bulk data. */
class FLiveLinkHubBulkData
{
public:
	/**
	 * Read a section of bulk data to a memory reader - valid only within this scope - allowing bulk data to be used like a typical FArchive.
	 * This differs from FBulkDataReader in that it supports only loading a section of bulk data into memory at a time.
	 */
	class FScopedBulkDataMemoryReader
	{
	public:
		FScopedBulkDataMemoryReader(const int64 InOffset, const int64 InBytesToRead, FLiveLinkHubBulkData* InBulkData);

		/** Retrieve the memory reader. */
		FLargeMemoryReader& GetMemoryReader() const { return *MemoryReader; }
		
		/** Retrieve the bulk data offset after having read the bulk data into memory. */
		int64 GetBulkDataOffset() const { return LocalBulkDataOffset; }
		
	private:
		/** The memory reader for reading out memory storage. */
		TUniquePtr<FLargeMemoryReader> MemoryReader;
		/** The memory storage for loading the bulk data. */
		TArray64<uint8> Memory;
		/** The bulk data offset after reading the data into memory. */
		int64 LocalBulkDataOffset = 0;
	};

	~FLiveLinkHubBulkData();

	/** Close the file reader if it is open. */
	void CloseFileReader();

	/** Unloads the bulk data. */
	void UnloadBulkData();
	
	/** Read data using sizeof(T). */
	template<typename T>
	void ReadBulkDataPrimitive(T& InMemory)
	{
		ReadBulkData(sizeof(T), reinterpret_cast<uint8*>(&InMemory));
	}

	/** Read bulk data at the current offset of the specified byte size and increment the file offset. */
	void ReadBulkData(const int64 InBytesToRead, uint8* InMemory);

	/** Create a scoped memory reader consisting of the bulk data bytes read. Increments file offset. */
	TSharedPtr<FScopedBulkDataMemoryReader> CreateBulkDataMemoryReader(const int64 InBytesToRead);
	
	/** Reset to the initial offset in the file. */
	void ResetBulkDataOffset();

	/** Manually set the bulk data offset. */
	void SetBulkDataOffset(const int64 InNewOffset);

	/** Retrieve the current bulk data offset. */
	int64 GetBulkDataOffset() const { return BulkDataOffset; }

	/** Call the serialize method of the bulk data, should be called when serializing the owning asset. */
	void Serialize(FArchive& Ar, UObject* Owner);

	/** Write to the bulk data file. */
	void WriteBulkData(TArray64<uint8>& Data);
	
private:
	/**
	 * Read bulk data at an offset of the specified byte size.
	 * @return The new offset after reading in the bytes.
	 */
	int64 ReadBulkDataImpl(int64 InOffset, int64 InBytesToRead, uint8* InMemory);
	
private:
	/** The file reader open to the bulk data. */
	TUniquePtr<IAsyncReadFileHandle> RecordingFileReader;
	/** The bulk data storage. */
	FByteBulkData BulkData;
	/** The current bulk data offset in the owning file. */
	int64 BulkDataOffset = 0;
};
