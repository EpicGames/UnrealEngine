// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHub.h"

#include "AboutScreen.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Clients/LiveLinkHubClientsController.h"
#include "Clients/LiveLinkHubProvider.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "HAL/PlatformProcess.h"
#include "ILiveLinkHubMessagingModule.h"
#include "ILiveLinkModule.h"
#include "ISettingsModule.h"
#include "LiveLinkEditorSettings.h"
#include "LiveLinkHubClient.h"
#include "LiveLinkHubCommands.h"
#include "LiveLinkHubCrashRecovery.h"
#include "LiveLinkHubCreatorAppMode.h"
#include "LiveLinkHubPlaybackAppMode.h"
#include "LiveLinkHubMessageBusSourceSettings.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubSubjectSettings.h"
#include "LiveLinkHubTicker.h"
#include "LiveLinkProviderImpl.h"
#include "LiveLinkSettings.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Settings/LiveLinkHubMediaProfileHandler.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "Recording/LiveLinkHubRecordingController.h"
#include "Recording/LiveLinkHubRecordingListController.h"
#include "Session/LiveLinkHubAutosaveHandler.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "Settings/LiveLinkHubSettings.h"
#include "Settings/LiveLinkHubTimeAndSyncSettings.h"
#include "Settings/LiveLinkHubTimingUtils.h"
#include "Subjects/LiveLinkHubSubjectController.h"
#include "UI/Window/LiveLinkHubWindowController.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub"

static TAutoConsoleVariable<bool> CVarLiveLinkHubEnableTriggeredTick(
	TEXT("LiveLinkHub.EnableTriggeredTick"), false,
	TEXT("Whether to allow LiveLinkHub's LiveLink client to tick whenever it receives new data. (Experimental)"),
	ECVF_RenderThreadSafe);

namespace LiveLinkHubInternalUtils
{
	/** Is this a standalone distributed build on the EGS? */
	bool IsDistributedBuild()
	{
		return FPaths::FileExists(TEXT("../../../LiveLinkHub/LiveLinkHub.uproject"));
	}

	/** Get the number of running LiveLinkHub instances. */
	uint32 GetInstanceCount()
	{
		FPlatformProcess::FProcEnumerator ProcIter;

		const FString ExePath = FPlatformProcess::ExecutablePath();
		const FString ExeFileName = FPaths::GetCleanFilename(ExePath);

		uint32 InstanceCount = 0;
		while (ProcIter.MoveNext())
		{
			FPlatformProcess::FProcEnumInfo ProcInfo = ProcIter.GetCurrent();

			if (ProcInfo.GetName() == ExeFileName)
			{
				InstanceCount++;
			}
		}

		return InstanceCount;
	}

	/** Generate the name of the livelinkhub provider. */
	FString GetProviderName(const FLiveLinkHubInstanceId& Id)
	{
		uint32 InstanceCount = GetInstanceCount();
		if (InstanceCount > 1)
		{
			return Id.ToString();
		}

		return TEXT("Live Link Hub");;
	}
}

TSharedPtr<FLiveLinkHub> FLiveLinkHub::Get()
{
	TSharedPtr<FLiveLinkHub> Hub;
	if (FLiveLinkHubModule* LiveLinkHubModule = FModuleManager::Get().GetModulePtr<FLiveLinkHubModule>("LiveLinkHub"))
	{
		Hub = LiveLinkHubModule->GetLiveLinkHub();
	}

	return Hub;
}

void FLiveLinkHub::Initialize(FLiveLinkHubTicker& Ticker)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHub::Initialize);
	
	TSharedRef<FLiveLinkHub> LiveLinkHub = SharedThis<FLiveLinkHub>(this);

	// We must register the livelink client first since we might rely on the modular feature to initialize the controllers/managers.
	if (GetDefault<ULiveLinkHubSettings>()->bTickOnGameThread)
	{
		LiveLinkHubClient = MakeShared<FLiveLinkHubClient>(LiveLinkHub);
	}
	else
	{
		LiveLinkHubClient = MakeShared<FLiveLinkHubClient>(LiveLinkHub, Ticker.OnTick());
	}

	IModularFeatures::Get().RegisterModularFeature(ILiveLinkClient::ModularFeatureName, LiveLinkHubClient.Get());

	SessionManager = MakeShared<FLiveLinkHubSessionManager>();

	LiveLinkProvider = MakeShared<FLiveLinkHubProvider>(SessionManager.ToSharedRef(), LiveLinkHubInternalUtils::GetProviderName(InstanceId));
	AutosaveHandler = MakeShared<FLiveLinkHubAutosaveHandler>();

	FModuleManager::Get().LoadModule("Settings");
	FModuleManager::Get().LoadModule("StatusBar");

	CommandExecutor = MakeUnique<FConsoleCommandExecutor>();
	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), CommandExecutor.Get());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHub::InitializeControllers);
		RecordingController = MakeShared<FLiveLinkHubRecordingController>();
		PlaybackController = MakeShared<FLiveLinkHubPlaybackController>();
		RecordingListController = MakeShared<FLiveLinkHubRecordingListController>(LiveLinkHub);
		ClientsController = MakeShared<FLiveLinkHubClientsController>(LiveLinkProvider.ToSharedRef());
		SubjectController = MakeShared<FLiveLinkHubSubjectController>();
	}
	
	if (FParse::Param(FCommandLine::Get(), TEXT("Hub")))
	{
		OverrideTopologyMode = ELiveLinkTopologyMode::Hub;
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("Spoke")))
	{
		OverrideTopologyMode = ELiveLinkTopologyMode::Spoke;
	}

	// We load it now to ensure we set the connection mode as early as possible to avoid it discovering instances by mistake.
	ILiveLinkHubMessagingModule* MessagingModule = static_cast<ILiveLinkHubMessagingModule*>(FModuleManager::Get().LoadModule("LiveLinkHubMessaging"));
	MessagingModule->SetHostTopologyMode(GetTopologyMode());

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLiveLinkHub::InitializePostEngineInit);
	FCoreDelegates::OnPreExit.AddRaw(this, &FLiveLinkHub::OnEnginePreExit);
}

void FLiveLinkHub::InitializePostEngineInit()
{
	FLiveLinkHubCommands::Register();

	TabManager = FGlobalTabmanager::Get();

	ILiveLinkHubMessagingModule& MessagingModule = FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");

	// Update the mode in our connection manager as well.
	MessagingModule.SetInstanceId(InstanceId);

	IModularFeatures::Get().OnModularFeatureRegistered().AddSP(this, &FLiveLinkHub::OnModularFeatureRegistered);

	WindowController = MakeShared<FLiveLinkHubWindowController>();
	
	WindowController->RestoreLayout(AsShared());
	
	// Note: Registering modes *must* happen after the layout was restored, since RestoreLayout will initiate the asset editor. 
	// Otherwise, the mode tabs will get registered with the global tab manager instead of the host app's tab manager.
	AddLiveLinkHubApplicationMode("CreatorMode", MakeShared<FLiveLinkHubCreatorAppMode>(SharedThis<FLiveLinkHubApplicationBase>(this)));
	AddLiveLinkHubApplicationMode("PlaybackMode", MakeShared<FLiveLinkHubPlaybackAppMode>(SharedThis<FLiveLinkHubApplicationBase>(this)));

	for (ILiveLinkHubApplicationModeFactory* Factory : IModularFeatures::Get().GetModularFeatureImplementations<ILiveLinkHubApplicationModeFactory>(ILiveLinkHubApplicationModeFactory::ModularFeatureName))
	{
		TSharedRef<FLiveLinkHubApplicationMode> AppMode = Factory->CreateLiveLinkHubAppMode(SharedThis<FLiveLinkHubApplicationBase>(this));
		AddLiveLinkHubApplicationMode(AppMode->GetModeName(), MoveTemp(AppMode));
	}

	SetCurrentMode("CreatorMode");

	DiscoverLayouts();

	LiveLinkHubClient->OnStaticDataReceived_AnyThread().AddSP(this, &FLiveLinkHub::OnStaticDataReceived_AnyThread);
	LiveLinkHubClient->OnFrameDataReceived_AnyThread().AddSP(this, &FLiveLinkHub::OnFrameDataReceived_AnyThread);
	LiveLinkHubClient->OnSubjectMarkedPendingKill_AnyThread().AddSP(this, &FLiveLinkHub::OnSubjectMarkedPendingKill_AnyThread);

	FDelegateHandle Handle1;
	FDelegateHandle Handle2;
	FLiveLinkSubjectKey GlobalKey = FLiveLinkSubjectKey{ FGuid(), FLiveLinkSubjectName{ UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN }};
	LiveLinkHubClient->RegisterForFrameDataReceived(GlobalKey, FOnLiveLinkSubjectStaticDataReceived::FDelegate(), FOnLiveLinkSubjectFrameDataReceived::FDelegate::CreateSP(this, &FLiveLinkHub::OnLiveLinkFrameDataReceived), Handle1, Handle2);

	RegisterLiveLinkHubSettings();

	PlaybackController->Start();

	// Create the handler that manages the transient MediaProfile and its session serialization.
	MediaProfileHandler = MakePimpl<FLiveLinkHubMediaProfileHandler>();

	// Timecode and genlock are now applied via the active MediaProfile.
	// Observe MediaProfile changes to re-broadcast settings to connected clients.
	MediaProfileChangedHandle = IMediaProfileManager::Get().OnMediaProfileChanged().AddSP(this, &FLiveLinkHub::OnMediaProfileChanged);

	// Also observe property edits within the active MediaProfile (e.g. changing the timecode provider's subject name).
	ObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FLiveLinkHub::OnObjectPropertyChanged);

	// Live Link Hub application broadcasts the MediaProfile's custom time step to its clients but should never run a custom timestep itself. 
	// Hook into GEngine's OnCustomTimeStepChanged so we can synchronously undo any application of the MediaProfile's custom time step inside the 
	// same call stack as UMediaProfile::Apply() — covering OnMediaProfileChanged, OnMediaProxiesChanged-style re-applies, and any other path 
	// that goes through GEngine->SetCustomTimeStep().
	if (GEngine)
	{
		CustomTimeStepChangedHandle = GEngine->OnCustomTimeStepChanged().AddSP(this, &FLiveLinkHub::OnEngineCustomTimeStepChanged);

		// Defend against: if a startup MediaProfile was already applied before this delegate was hooked,
		// clear it now. Subsequent applications of custom timestep will be caught by the delegate.
		UE::LiveLinkHubTimingUtils::Private::ClearMediaProfileCustomTimeStepFromEngine();
	}

	ILiveLinkModule& LiveLinkModule = FModuleManager::Get().GetModuleChecked<ILiveLinkModule>("LiveLink");
	LiveLinkModule.OnSubjectOutboundNameModified().AddSP(this, &FLiveLinkHub::OnSubjectOutboundNameModified);

	GIsRunning = true;

	if (GetDefault<ULiveLinkHubSettings>()->bEnableCrashRecovery)
	{
		CrashRecoveryHandler = MakePimpl<FLiveLinkHubCrashRecovery>();
	}

	FString SessionPath;
	if (FParse::Value(FCommandLine::Get(), TEXT("SessionPath="), SessionPath) && SessionPath.Len())
	{
		SessionManager->RestoreSession(SessionPath);
	}
	else if (!GetDefault<ULiveLinkHubSettings>()->StartupConfig.FilePath.IsEmpty())
	{ 
		SessionManager->RestoreSession(GetDefault<ULiveLinkHubSettings>()->StartupConfig.FilePath);
	}

	// Close the playback tab when starting the app in case LLH was closed with the tab opened.
	if (TSharedPtr<SDockTab> PlaybackTab = FLiveLinkHub::Get()->GetTabManager()->FindExistingLiveTab(UE::LiveLinkHub::PlaybackTabId))
	{
		PlaybackTab->RequestCloseTab();
	}
}

void FLiveLinkHub::OnEnginePreExit()
{
	if (PlaybackController)
	{
		if (ULiveLinkUAssetRecording* Recording = Cast<ULiveLinkUAssetRecording>(PlaybackController->GetRecording().Get()))
		{
			Recording->UnloadRecordingData();
		}
	}

	// Tearing down modes before engine tears down singleton objects
	if (CurrentAppModePtr.IsValid())
	{
		CurrentAppModePtr->PreDeactivateMode();
	}
	
	RemoveAllApplicationModes();
	CurrentAppModePtr.Reset();
}

FLiveLinkHub::~FLiveLinkHub()
{
	if (MediaProfileChangedHandle.IsValid()
		&& FModuleManager::Get().IsModuleLoaded(TEXT("MediaProfile")))
	{
		IMediaProfileManager::Get().OnMediaProfileChanged().Remove(MediaProfileChangedHandle);
	}

	if (ObjectPropertyChangedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(ObjectPropertyChangedHandle);
	}

	if (CustomTimeStepChangedHandle.IsValid() && GEngine)
	{
		GEngine->OnCustomTimeStepChanged().Remove(CustomTimeStepChangedHandle);
	}

	// Destroy the media profile handler before other systems are torn down.
	MediaProfileHandler.Reset();

	IModularFeatures::Get().UnregisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), CommandExecutor.Get());
	CommandExecutor.Reset();

	UnregisterLiveLinkHubSettings();

	RecordingController.Reset();
	PlaybackController.Reset();

	LiveLinkHubClient->OnSubjectMarkedPendingKill_AnyThread().RemoveAll(this);
	LiveLinkHubClient->OnFrameDataReceived_AnyThread().RemoveAll(this);
	LiveLinkHubClient->OnStaticDataReceived_AnyThread().RemoveAll(this);

	IModularFeatures::Get().UnregisterModularFeature(ILiveLinkClient::ModularFeatureName, LiveLinkHubClient.Get());

	FCoreDelegates::OnPreExit.RemoveAll(this);
}

void FLiveLinkHub::OnMediaProfileChanged(UMediaProfile* PreviousProfile, UMediaProfile* NewProfile)
{
	// When the active MediaProfile changes, re-broadcast timecode/genlock settings to all connected clients.
	UE::LiveLinkHubTimingUtils::Private::BroadcastCurrentSettings();
}

void FLiveLinkHub::OnEngineCustomTimeStepChanged()
{
	// Synchronously undo the MediaProfile's application of its custom time step on GEngine.
	// This fires inside GEngine->SetCustomTimeStep() before any tick can observe the new value.
	UE::LiveLinkHubTimingUtils::Private::ClearMediaProfileCustomTimeStepFromEngine();
}

void FLiveLinkHub::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!Object || PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified)
	{
		return;
	}

	const UMediaProfile* ActiveProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveProfile)
	{
		return;
	}

	// Check if the changed object is the active MediaProfile itself, or one of its timecode/genlock subobjects.
	const bool bIsActiveProfile = (Object == ActiveProfile);
	const bool bIsTimecodeProvider = (ActiveProfile->GetTimecodeProvider() && Object == ActiveProfile->GetTimecodeProvider());
	const bool bIsCustomTimeStep = (ActiveProfile->GetCustomTimeStep() && Object == ActiveProfile->GetCustomTimeStep());

	if (bIsActiveProfile || bIsTimecodeProvider || bIsCustomTimeStep)
	{
		UE::LiveLinkHubTimingUtils::Private::BroadcastCurrentSettings();
	}
}

bool FLiveLinkHub::IsInPlayback() const
{
	return PlaybackController->IsInPlayback();
}

bool FLiveLinkHub::IsRecording() const
{
	return RecordingController->IsRecording();
}

TSharedRef<SWindow> FLiveLinkHub::GetRootWindow() const
{
	return WindowController->GetRootWindow().ToSharedRef();
}

TSharedPtr<FLiveLinkHubProvider> FLiveLinkHub::GetLiveLinkProvider() const
{
	return LiveLinkProvider;
}

TSharedPtr<FLiveLinkHubClientsController> FLiveLinkHub::GetClientsController() const
{
	return ClientsController;
}

TSharedPtr<ILiveLinkHubSessionManager> FLiveLinkHub::GetSessionManager() const
{
	return SessionManager;
}

TSharedPtr<FLiveLinkHubAutosaveHandler> FLiveLinkHub::GetAutosaveHandler() const
{
	return AutosaveHandler;
}

TSharedPtr<FLiveLinkHubRecordingController> FLiveLinkHub::GetRecordingController() const
{
	return RecordingController;
}

TSharedPtr<FLiveLinkHubRecordingListController> FLiveLinkHub::GetRecordingListController() const
{
	return RecordingListController;
}

TSharedPtr<FLiveLinkHubPlaybackController> FLiveLinkHub::GetPlaybackController() const
{
	return PlaybackController;
}

ELiveLinkTopologyMode FLiveLinkHub::GetTopologyMode() const
{
	if (OverrideTopologyMode)
	{
		return *OverrideTopologyMode;
	}

	if (SessionManager)
	{
		if (TSharedPtr<ILiveLinkHubSession> CurrentSession = SessionManager->GetCurrentSession())
		{
			return CurrentSession->GetTopologyMode();
		}
	}

	return ELiveLinkTopologyMode::Hub;
}

void FLiveLinkHub::SetTopologyMode(ELiveLinkTopologyMode Mode)
{
	checkf(!OverrideTopologyMode, TEXT("Can't set topology mode at runtime if it was set through the command line."));

	if (Mode != GetTopologyMode())
	{
		ILiveLinkHubMessagingModule& MessagingModule = FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");

		// Update the mode in our connection manager as well.
		MessagingModule.SetHostTopologyMode(Mode);

		if (SessionManager)
		{
			if (TSharedPtr<ILiveLinkHubSession> CurrentSession = SessionManager->GetCurrentSession())
			{
				CurrentSession->SetTopologyMode(Mode);
			}
		}

		if (Mode == ELiveLinkTopologyMode::Spoke)
		{
			// Clear out the list of LLH sources if we're switching from a hub to a spoke.
			constexpr bool bEvenIfPendingKill = true;
			for (const FGuid& Source : LiveLinkHubClient->GetSources(bEvenIfPendingKill))
			{
				if (ULiveLinkSourceSettings* Settings = LiveLinkHubClient->GetSourceSettings(Source))
				{
					if (Settings->IsA<ULiveLinkHubMessageBusSourceSettings>())
					{
						LiveLinkHubClient->RemoveSource(Source);
					}
				}
			}
		}

		TopologyModeChangedDelegate.Broadcast(Mode);
	}
}

void FLiveLinkHub::ToggleTopologyMode()
{
	ELiveLinkTopologyMode CurrentMode = GetTopologyMode();
	if (CurrentMode == ELiveLinkTopologyMode::Hub)
	{
		SetTopologyMode(ELiveLinkTopologyMode::Spoke);
	}
	else
	{
		SetTopologyMode(ELiveLinkTopologyMode::Hub);
	}
}

bool FLiveLinkHub::CanSetTopologyMode() const
{
	// Can't modify topology if it was overridden through the command line.
	return !OverrideTopologyMode.IsSet();
}

void FLiveLinkHub::UpgradeAndSaveRecording(ULiveLinkUAssetRecording* InRecording, bool bUpgradeInPlace)
{
	GetPlaybackController()->OnPlaybackReady().AddSP(this, &FLiveLinkHub::OnPlaybackReadyForUpgrade, bUpgradeInPlace);
	GetPlaybackController()->PreparePlayback(InRecording);
}

void FLiveLinkHub::MapToolkitCommands()
{
	const FLiveLinkHubCommands& Commands = FLiveLinkHubCommands::Get();
	TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	CommandList->MapAction(Commands.NewConfig, FExecuteAction::CreateSP(this, &FLiveLinkHub::NewConfig),
		FCanExecuteAction::CreateSP(this, &FLiveLinkHub::CanCreateOrOpenConfig));
	CommandList->MapAction(Commands.OpenConfig, FExecuteAction::CreateSP(this, &FLiveLinkHub::OpenConfig),
		FCanExecuteAction::CreateSP(this, &FLiveLinkHub::CanCreateOrOpenConfig));
	CommandList->MapAction(Commands.SaveConfigAs, FExecuteAction::CreateSP(this, &FLiveLinkHub::SaveConfigAs));
	CommandList->MapAction(Commands.SaveConfig, FExecuteAction::CreateSP(this, &FLiveLinkHub::SaveConfig),
		FCanExecuteAction::CreateSP(this, &FLiveLinkHub::CanSaveConfig));
	CommandList->MapAction(Commands.OpenLogsFolder, FExecuteAction::CreateSP(this, &FLiveLinkHub::OpenLogsFolder));
	CommandList->MapAction(Commands.OpenAboutMenu, FExecuteAction::CreateSP(this, &FLiveLinkHub::OpenAboutMenu));

	FLiveLinkHubApplication::MapToolkitCommands();
}

void FLiveLinkHub::OnClose()
{
	// Make sure to send a disconnect message to every UE Client to ensure that they remove their LiveLinkHub source.
	if (TSharedPtr<ILiveLinkHubSession> CurrentSession = SessionManager->GetCurrentSession())
	{
		// Note: This has to be done before the LiveLink Provider is destroyed.
		CurrentSession->RemoveAllClients();
	}

	FWorkflowCentricApplication::OnClose();
}

void FLiveLinkHub::OnStaticDataReceived_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, const FLiveLinkStaticDataStruct& InStaticDataStruct) const
{
	if (RecordingController->IsRecording())
	{
		RecordingController->RecordStaticData(InSubjectKey, InRole, InStaticDataStruct);
	}
}

void FLiveLinkHub::OnLiveLinkFrameDataReceived(const FLiveLinkFrameDataStruct&) const
{
	if (CVarLiveLinkHubEnableTriggeredTick.GetValueOnAnyThread())
	{
		OnFrameReceivedDelegate.ExecuteIfBound();
	}
}


void FLiveLinkHub::OnFrameDataReceived_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, const FLiveLinkFrameDataStruct& InFrameDataStruct) const
{
	if (RecordingController->IsRecording())
	{
		RecordingController->RecordFrameData(InSubjectKey, InFrameDataStruct);
	}
}

void FLiveLinkHub::OnSubjectMarkedPendingKill_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) const
{
	UE_LOGF(LogLiveLinkHub, Verbose, "Removed subject %ls", *InSubjectKey.SubjectName.ToString());

	// Send an update to connected clients as well.
	const FName OverridenName = LiveLinkHubClient->GetRebroadcastName(InSubjectKey);

	// Note: We send a RemoveSubject message to connected clients when the subject is marked pending kill in order to process this message in the right order.
	// If we were to send a RemoveSubject message after the OnSubjectRemoved delegate, it could cause our RemoveSubject message to be sent out of order.
	LiveLinkProvider->RemoveSubject(OverridenName);
}

void FLiveLinkHub::NewConfig()
{
	SessionManager->NewSession();
}

void FLiveLinkHub::SaveConfigAs()
{
	SessionManager->SaveSessionAs();
}

bool FLiveLinkHub::CanSaveConfig() const
{
	return SessionManager->CanSaveCurrentSession();
}

void FLiveLinkHub::SaveConfig()
{
	SessionManager->SaveCurrentSession();
}

void FLiveLinkHub::OpenConfig()
{
	SessionManager->RestoreSession();
}

bool FLiveLinkHub::CanCreateOrOpenConfig() const
{
	return !PlaybackController->IsInPlayback();
}

void FLiveLinkHub::OpenLogsFolder()
{
	const FString AbsoluteFilePath = FPaths::ConvertRelativePathToFull(FGenericPlatformOutputDevices::GetAbsoluteLogFilename());
	const FString FileDirectory = FPaths::GetPath(AbsoluteFilePath);
	FPlatformProcess::ExploreFolder(*FileDirectory);
}

void FLiveLinkHub::OpenAboutMenu()
{
	const FText AboutWindowTitle = LOCTEXT("AboutLiveLinkHub", "About Live Link Hub");

	TSharedPtr<SWindow> AboutWindow =
		SNew(SWindow)
		.Title(AboutWindowTitle)
		.ClientSize(FVector2D(720.f, 538.f))
		.SupportsMaximize(false).SupportsMinimize(false)
		.SizingRule(ESizingRule::FixedSize)
		[
			SNew(SAboutScreen)
		];

	FSlateApplication::Get().AddWindow(AboutWindow.ToSharedRef());
}

void FLiveLinkHub::OnModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == ILiveLinkHubApplicationModeFactory::ModularFeatureName)
	{
		ILiveLinkHubApplicationModeFactory* Factory = static_cast<ILiveLinkHubApplicationModeFactory*>(ModularFeature);

		TSharedRef<FLiveLinkHubApplicationMode> AppMode = Factory->CreateLiveLinkHubAppMode(SharedThis<FLiveLinkHubApplicationBase>(this));
		AddLiveLinkHubApplicationMode(AppMode->GetModeName(), MoveTemp(AppMode));
	}
}

void FLiveLinkHub::OnSubjectOutboundNameModified(const FLiveLinkSubjectKey& SubjectKey, const FString& PreviousOutboundName, const FString& NewOutboundName) const
{
	LiveLinkProvider->SendClearSubjectToConnections(*PreviousOutboundName);

	FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

	// Re-send the last static data with the new name.
	TPair<UClass*, FLiveLinkStaticDataStruct*> StaticData = LiveLinkProvider->GetLastSubjectStaticDataStruct(*PreviousOutboundName);
	if (StaticData.Key && StaticData.Value)
	{
		FLiveLinkStaticDataStruct StaticDataCopy;
		StaticDataCopy.InitializeWith(*StaticData.Value);

		TMap<FName, FString> ExtraAnnotations;

		const FText OriginalSourceType = LiveLinkHubClient->GetSourceType(SubjectKey.Source);
		ExtraAnnotations.Add(FLiveLinkMessageAnnotation::OriginalSourceAnnotation, OriginalSourceType.ToString());
		LiveLinkProvider->UpdateSubjectStaticData(*NewOutboundName, StaticData.Key, MoveTemp(StaticDataCopy), ExtraAnnotations);
	}

	// Then clear the old static data entry in the provider.
	LiveLinkProvider->RemoveSubject(*PreviousOutboundName);
}

void FLiveLinkHub::OnPlaybackReadyForUpgrade(bool bUpgradeInPlace)
{
	if (!PlaybackController || !RecordingController || !PlaybackController->GetRecording())
	{
		return;
	}

	PlaybackController->OnPlaybackReady().RemoveAll(this);

	FString RecordingPath = PlaybackController->GetRecording()->GetPackage()->GetPathName();
	if (!bUpgradeInPlace)
	{
		// Get the save path before starting the long task.
		const FString DefaultName = PlaybackController->GetRecording()->GetName() + TEXT("_copy");
		if (!RecordingController->OpenSaveAsPrompt(RecordingPath, DefaultName))
		{
			PlaybackController->EjectAndUnload();
			return;
		}
	}
		
	ULiveLinkUAssetRecording* CurrentRecording = CastChecked<ULiveLinkUAssetRecording>(PlaybackController->GetRecording().Get());
		
	TWeakObjectPtr<ULiveLinkUAssetRecording> CurrentRecordingWeakPtr(CurrentRecording);
	CurrentRecording->GetOnRecordingFullyLoaded().BindLambda([this, CurrentRecordingWeakPtr, RecordingPath, bUpgradeInPlace]()
	{
		const TStrongObjectPtr<ULiveLinkUAssetRecording> CurrentRecordingPin = CurrentRecordingWeakPtr.Pin();
		if (CurrentRecordingPin.IsValid() && RecordingController)
		{
			RecordingController->RecordFromExistingAndSave(CurrentRecordingPin.Get(), RecordingPath, bUpgradeInPlace);
		}
	});

	CurrentRecording->LoadEntireRecordingForUpgrade();
}

void FLiveLinkHub::RegisterLiveLinkHubSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "Live Link",
			LOCTEXT("EditorSettingsName", "Live Link"),
			LOCTEXT("EditorSettingsDescription", "Configure Live Link."),
			GetMutableDefault<ULiveLinkEditorSettings>()
		);

		SettingsModule->RegisterSettings("Project", "Plugins", "Live Link",
			LOCTEXT("LiveLinkSettingsName", "Live Link"),
			LOCTEXT("LiveLinkDescription", "Configure Live Link."),
			GetMutableDefault<ULiveLinkSettings>()
		);

		SettingsModule->RegisterSettings("Project", "Plugins", "Live Link Hub",
			LOCTEXT("LiveLinkHubSettingsName", "Live Link Hub"),
			LOCTEXT("LiveLinkHubDescription", "Configure Live Link Hub."),
			GetMutableDefault<ULiveLinkHubSettings>()
		);

		SettingsModule->RegisterSettings("Project", "Application", "Timing & Sync",
			LOCTEXT("LiveLinkHubTimeAndSyncName", "Timing & Sync"),
			LOCTEXT("LiveLinkHubTimeAndSyncNameDescription", "Configure Live Link Hub timecode and genlock settings."),
			GetMutableDefault<ULiveLinkHubTimeAndSyncSettings>()
		);
	}
}

void FLiveLinkHub::UnregisterLiveLinkHubSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "Live Link");
		SettingsModule->UnregisterSettings("Project", "Plugins", "Live Link");
		SettingsModule->UnregisterSettings("Project", "Plugins", "Live Link Hub");
		SettingsModule->UnregisterSettings("Project", "Application", "Timing & Sync");
	}
}

#undef LOCTEXT_NAMESPACE /*LiveLinkHub*/
