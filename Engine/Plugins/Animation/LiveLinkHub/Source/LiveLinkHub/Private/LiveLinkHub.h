// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubApplication.h"

#include "ILiveLinkHubMessagingModule.h"
#include "LiveLinkTypes.h"
#include "LiveLinkHubMessages.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Toolkits/FConsoleCommandExecutor.h"

class FLiveLinkHubAutosaveHandler;
class FLiveLinkHubMediaProfileHandler;
class FLiveLinkHubClient;
class FLiveLinkHubClientsController;
class FLiveLinkHubPlaybackController;
class FLiveLinkHubRecordingController;
class FLiveLinkHubRecordingListController;
class FLiveLinkHubSubjectController;
class FLiveLinkHubWindowController;
class FLiveLinkHubProvider;
class ILiveLinkHubSessionManager;
class SWindow;
class ULiveLinkRole;
class UMediaProfile;
class ULiveLinkUAssetRecording;
struct FLiveLinkSubjectKey;

DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnTopologyModeChanged, ELiveLinkTopologyMode /*NewMode*/);

/**
 * Main interface for the live link hub.
 */
class ILiveLinkHub
{
public:
	virtual ~ILiveLinkHub() = default;

	/** Whether the hub is currently playing a recording. */
	virtual bool IsInPlayback() const = 0;
	/** Whether the hub is currently recording livelink data. */
	virtual bool IsRecording() const = 0;
};

/**
 * Implementation of the live link hub.
 * Contains the apps' different components and is responsible for handling communication between them.
 */
class FLiveLinkHub : public ILiveLinkHub, public FLiveLinkHubApplication
{
public:
	/** Get the livelinkhub instance that's held by LiveLinkHubModule. May return a nullptr when the module is shutting down. */ 
	static TSharedPtr<FLiveLinkHub> Get();

	virtual ~FLiveLinkHub() override;

	//~ Begin ILiveLinkHub interface
	virtual bool IsInPlayback() const override;
	virtual bool IsRecording() const override;
	//~ End ILiveLinkHub interface


public:
	/** 
	 * Initialize LiveLinkHub. */
	void Initialize(class FLiveLinkHubTicker& Ticker);
	const FLiveLinkHubInstanceId& GetId() const { return InstanceId; }
	/** Get the root window that hosts the hub's slate application. */
	TSharedRef<SWindow> GetRootWindow() const;
	/** Get the livelink provider used to rebroadcast livelink data to connected UE clients. */
	TSharedPtr<FLiveLinkHubProvider> GetLiveLinkProvider() const;
	/** Get the controller that manages recording livelink data. */
	TSharedPtr<FLiveLinkHubRecordingController> GetRecordingController() const;
	/** Get the recording list controller, that handles displaying livelink recording assets. */
	TSharedPtr<FLiveLinkHubRecordingListController> GetRecordingListController() const;
	/** Get the controller that manages playing back livelink data. */
	TSharedPtr<FLiveLinkHubPlaybackController> GetPlaybackController() const;
	/** Get the controller that manages clients. */
	TSharedPtr<FLiveLinkHubClientsController> GetClientsController() const;
	/** Get the live link hub command list. */
	TSharedPtr<FUICommandList> GetCommandList() const { return GetToolkitCommands(); }
	/** Get the session manager. */
	TSharedPtr<ILiveLinkHubSessionManager> GetSessionManager() const;
	/** Get the autosave handler. */
	TSharedPtr<FLiveLinkHubAutosaveHandler> GetAutosaveHandler() const;
	/** 
	 * Get whether the hub should act as a hub or as a spoke. 
	 * Hubs can receive data from spokes, and transmit it to UE clients.
	 * Spokes can only transmit data to Hubs.
	 */
	ELiveLinkTopologyMode GetTopologyMode() const;
	/** Set the topology mode for this instance. */
	void SetTopologyMode(ELiveLinkTopologyMode Mode);
	/** Toggle the topology mode between Hub and Spoke. */
	void ToggleTopologyMode();
	/** Returns whether the topology mode can change for this instance. */
	bool CanSetTopologyMode() const;
	/** Delegate called when the topology mode changed. */
	FOnTopologyModeChanged& OnTopologyModeChanged() { return TopologyModeChangedDelegate; }

	/** 
	 * Upgrade a recording and save to a new file or over the existing file.
	 */
	void UpgradeAndSaveRecording(ULiveLinkUAssetRecording* InRecording, bool bUpgradeInPlace = false);

	/** Delegate triggered whenever the Live Link Client receives frame data. */
	FSimpleDelegate& OnFrameReceived() { return OnFrameReceivedDelegate; }

	//~ FAssetEditorToolkit interface
	virtual void MapToolkitCommands() override;
	virtual void OnClose() override;

private:
	/** Complete the initialization after core engine systems have been initialized. */
	void InitializePostEngineInit();

	/** When the engine is about to exit. */
	void OnEnginePreExit();

	//~ LiveLink Client delegates
	void OnStaticDataReceived_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, const FLiveLinkStaticDataStruct& InStaticDataStruct) const;
	void OnFrameDataReceived_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, const FLiveLinkFrameDataStruct& InFrameDataStruct) const;
	void OnSubjectMarkedPendingKill_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) const;

	/** Called whenever LiveLinkClient receives data to wake up the LiveLink Ticker if LiveLinkHub.EnableTriggeredTick is enabled. */
	void OnLiveLinkFrameDataReceived(const FLiveLinkFrameDataStruct&) const;
	//~ LiveLink Client delegates

	/** Register settings relevant to the livelink hub. */
	void RegisterLiveLinkHubSettings();
	/** Unregister settings relevant to the livelink hub. */
	void UnregisterLiveLinkHubSettings();

	/** Create a new config. */
	void NewConfig();

	/** Save an existing config to a new file. */
	void SaveConfigAs();

	/** Whether the Save command can be used. */
	bool CanSaveConfig() const;
	
	/** Save the config to the current file. */
	void SaveConfig();
	
	/** Open an existing config. */
	void OpenConfig();

	/** If a config can be created or loaded. */
	bool CanCreateOrOpenConfig() const;

	/** Open the folder containing logs for the app. */
	void OpenLogsFolder();

	/** Open the help menu. */
	void OpenAboutMenu();

	/** Handles adding application modes that are registered after LLH has finished initialization. */
	void OnModularFeatureRegistered(const FName& Type, class IModularFeature* ModularFeature);

	/** Delegate called when a LiveLinkHubSubjectSettings modifies an outbound name. We need to notify connected clients of this change. */
	void OnSubjectOutboundNameModified(const FLiveLinkSubjectKey& SubjectKey, const FString& PreviousOutboundName, const FString& NewOutboundName) const;

	/** Callback when playback is ready because we are performing an upgrade. */
	void OnPlaybackReadyForUpgrade(bool bUpgradeInPlace);
	
private:
	/** Implements the logic to manage the clients tabs. */
	TSharedPtr<FLiveLinkHubClientsController> ClientsController;
	/**  Implements the logic for triggering recording. */
	TSharedPtr<FLiveLinkHubRecordingController> RecordingController;
	/** Implements the logic for displaying the list of recordings. */
	TSharedPtr<FLiveLinkHubRecordingListController> RecordingListController;
	/** Implements the logic for triggering the playback of a livelink recording. */
	TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController;
	/** Implements the controller responsible for displaying and managing subject data. */
	TSharedPtr<FLiveLinkHubSubjectController> SubjectController;
	/** Controller responsible for creating and managing the app's slate windows. */
	TSharedPtr<FLiveLinkHubWindowController> WindowController;
	/** Object responsible for managing sessions.  */
	TSharedPtr<ILiveLinkHubSessionManager> SessionManager;
	/** LiveLinkHub's livelink client. */
	TSharedPtr<FLiveLinkHubClient> LiveLinkHubClient;
	/** LiveLinkProvider used to transfer data to connected UE clients. */
	TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider;
	/** Handles execution of commands. */
	TUniquePtr<FConsoleCommandExecutor> CommandExecutor;
	/** Handles triggering an autosave. */
	TSharedPtr<FLiveLinkHubAutosaveHandler> AutosaveHandler;
	/** Crash recovery handler. */
	TPimplPtr<class FLiveLinkHubCrashRecovery> CrashRecoveryHandler;
	/** The last opened config path. */
	FString LastConfigPath;
	/** LiveLinkHub instance Id, used to disambiguate different instances. */
	FLiveLinkHubInstanceId InstanceId = FLiveLinkHubInstanceId{ FGuid::NewGuid() };
	/** Override topology mode set through the command line. Cannot be changed at runtime. */
	TOptional<ELiveLinkTopologyMode> OverrideTopologyMode;
	/** Delegate called when the topology mode changed. */
	FOnTopologyModeChanged TopologyModeChangedDelegate;
	/** Delegate called whenever LL Client receives a frame. */
	FSimpleDelegate OnFrameReceivedDelegate;
	/** Handle for media profile changed delegate. */
	FDelegateHandle MediaProfileChangedHandle;
	/** Handle for object property changed delegate (to detect edits within the active MediaProfile). */
	FDelegateHandle ObjectPropertyChangedHandle;
	/** Handle for engine custom time step changed delegate (used to keep the Hub off the MediaProfile's genlock). */
	FDelegateHandle CustomTimeStepChangedHandle;

	/** Manages the transient MediaProfile and its session serialization. */
	TPimplPtr<FLiveLinkHubMediaProfileHandler> MediaProfileHandler;

	/** Called when the active MediaProfile changes. Re-broadcasts timecode/genlock settings to connected clients. */
	void OnMediaProfileChanged(UMediaProfile* PreviousProfile, UMediaProfile* NewProfile);

	/** Called when any UObject property changes. Filters for the active MediaProfile and its timecode/genlock subobjects. */
	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	/**
	 * Called when GEngine's custom time step changes. Hub shouldn't ever run with a custom timestep,
	 * so when the MediaProfile installs its custom time step we synchronously clear it.
	 */
	void OnEngineCustomTimeStepChanged();

	friend class FLiveLinkHubModule;
};
