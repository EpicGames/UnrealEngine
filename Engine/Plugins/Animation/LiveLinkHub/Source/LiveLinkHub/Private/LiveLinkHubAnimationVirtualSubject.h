// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkAnimationVirtualSubject.h"

#include "Clients/LiveLinkHubProvider.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubSubjectSettingsUtils.h"

#include "LiveLinkHubAnimationVirtualSubject.generated.h"

/**
 * Animation virtual subject used in LiveLinkHub.
 * Shows options for the subject and broadcasts static data when the skeleton is updated.
 */
UCLASS()
class ULiveLinkHubAnimationVirtualSubject : public ULiveLinkAnimationVirtualSubject
{
	GENERATED_BODY()

public:
	virtual void Initialize(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* InLiveLinkClient) override
	{
		ULiveLinkAnimationVirtualSubject::Initialize(InSubjectKey, InRole, InLiveLinkClient);

		if (OutboundName.IsEmpty())
		{
			OutboundName = InSubjectKey.SubjectName.ToString();
		}

		FLiveLinkClient* Client = &IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(FLiveLinkClient::ModularFeatureName);
		Source = Client->GetSourceType(InSubjectKey.Source).ToString();
	}

	virtual FText GetDisplayName() const override
	{
		// In the case of Virtual Subjects, we don't care about the original name.
		return FText::FromString(OutboundName);
	}

	virtual FName GetRebroadcastName() const override
	{
		return *OutboundName;
	}

	/** Whether this subject is rebroadcasted */
	virtual bool IsRebroadcasted() const
	{
		// todo: Decide this based on the session? Provider should decide how to handle it .
		return true;
	}

	virtual void PostSkeletonRebuild() override
	{
		if (HasValidStaticData())
		{
			if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkProvider())
			{
				// Update the static data since the final skeleton was changed.
				const FLiveLinkSubjectFrameData& CurrentSnapshot = GetFrameSnapshot();

				FLiveLinkStaticDataStruct StaticDataCopy;
				StaticDataCopy.InitializeWith(CurrentSnapshot.StaticData);

				LiveLinkProvider->UpdateSubjectStaticData(GetRebroadcastName(), Role, MoveTemp(StaticDataCopy));
			}
		}
	}

	//~ Begin UObject interface
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override
	{
		Super::PreEditChange(PropertyAboutToChange);

		if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubAnimationVirtualSubject, OutboundName))
		{
			PreviousOutboundName = *OutboundName;
		}
	}

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubAnimationVirtualSubject, OutboundName))
		{
			if (PreviousOutboundName != *OutboundName)
			{
				if (FLiveLinkHubSubjectSettingsUtils::ValidateOutboundName(OutboundName, PreviousOutboundName, OutboundName))
				{
					FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
					if (TSharedPtr<FLiveLinkHubProvider> Provider = LiveLinkHubModule.GetLiveLinkProvider())
					{
						Provider->SendClearSubjectToConnections(PreviousOutboundName);
					}

					FLiveLinkHubSubjectSettingsUtils::NotifyRename(PreviousOutboundName, OutboundName, SubjectKey);
				}
				else
				{
					OutboundName = PreviousOutboundName.ToString();
				}
			}
		}
	}
	//~ End UObject interface

public:
	/** Name of the virtual subject. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	FString OutboundName;

	/** Source that contains the subject. */
	UPROPERTY(VisibleAnywhere, Category = "LiveLink")
	FString Source;

private:
	/* Previous outbound name to be used for reverting name changes that aren't valid. */
	FName PreviousOutboundName;
};
