// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubApplication.h"

#include "Algo/RemoveIf.h"
#include "Dom/JsonObject.h"
#include "HAL/FileManagerGeneric.h"
#include "LiveLinkHubCreatorAppMode.h"
#include "LiveLinkHubUserLayoutMode.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeExit.h"
#include "Serialization/JsonSerializer.h"
#include "Session/LiveLinkHubAutosaveHandler.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ToolMenus.h"
#include "UI/Widgets/SApplicationModeSwitcher.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#if !UE_BUILD_SHIPPING
#	if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
#		include "ISessionFrontendModule.h"
#	endif
#	include "ISlateReflectorModule.h"
#	include "IMessageBusTesterEditorModule.h"
#endif

#define LOCTEXT_NAMESPACE "LiveLinkHubApplication"

struct FLiveLinkHubUserLayout
{
	FString ParentModeName;
	TSharedPtr<FJsonObject> JsonLayout;
};

namespace LiveLinkHubLayoutFileUtils
{
	const FString LayoutFileExtension = TEXT("llhlayout");
	static const FString LayoutDescription = TEXT("Live Link Hub Layout");
	static const FString LayoutExtension = LayoutFileExtension;
	static const FString LayoutDefaultFileName = TEXT("New Layout");

	static const FString FileTypes = FString::Printf(TEXT("%s (*.%s)|*.%s"), *LayoutDescription, *LayoutExtension, *LayoutExtension);
}

FLiveLinkHubApplication::FLiveLinkHubApplication()
	: FLiveLinkHubApplicationBase()
{
	LayoutIni = TEXT("LiveLinkHubLayout");

	FEditorDirectories::Get().LoadLastDirectories();

	// We set an extender here and collect toolbar widgets 
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::Before,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateRaw(this, &FLiveLinkHubApplication::AddToolbarExtenders)
	);

	AddToolbarExtender(Extender);
}

void FLiveLinkHubApplication::AddToolbarExtenders(FToolBarBuilder&)
{
	if (TSharedPtr<FApplicationMode> Mode = GetCurrentModePtr())
	{
		TArray<TSharedRef<SWidget>> Widgets = StaticCastSharedPtr<FLiveLinkHubApplicationMode>(GetCurrentModePtr())->GetToolbarWidgets();

		for (const TSharedRef<SWidget>& Widget : Widgets)
		{
			AddToolbarWidget(Widget);
		}
	}
}

void FLiveLinkHubApplication::AddLiveLinkHubApplicationMode(FName ModeName, TSharedRef<FLiveLinkHubApplicationMode> Mode)
{
	CachedModeInfo.Add(ModeName, FLiveLinkHubAppModeInfo{ Mode->GetModeIcon(), Mode->GetDisplayName(), Mode->IsUserLayout()});

	// Modes may provide a custom Window menu section via GetCustomWindowMenuCategory().
	// If none is provided, all tabs fall under the shared "Live Link Hub" section.
	// The result is cached here so PushTabFactories uses the same instance that was registered.
	TSharedPtr<FWorkspaceItem> ModeCategory = Mode->GetCustomWindowMenuCategory();
	CachedModeWorkspaceCategories.Add(ModeName, ModeCategory);
	if (ModeCategory.IsValid())
	{
		TabManager->AddLocalWorkspaceMenuItem(ModeCategory.ToSharedRef());
	}
	else if (!DataViewsWorkspaceCategory.IsValid())
	{
		DataViewsWorkspaceCategory = FWorkspaceItem::NewGroup(LOCTEXT("LiveLinkHubCategoryLabel", "Live Link Hub"));
		TabManager->AddLocalWorkspaceMenuItem(DataViewsWorkspaceCategory.ToSharedRef());
	}

	AddApplicationMode(ModeName, MoveTemp(Mode));
}

void FLiveLinkHubApplication::RemoveLiveLinkHubApplicationMode(FName ModeName)
{
	CachedModeInfo.Remove(ModeName);
	CachedModeWorkspaceCategories.Remove(ModeName);
	RemoveApplicationMode(ModeName);
}

void FLiveLinkHubApplication::PushTabFactories(const FWorkflowAllowedTabSet& FactorySetToPush, TSharedPtr<FLiveLinkHubApplicationMode> ApplicationMode)
{
	check(TabManager.IsValid());

	// Use the category cached at registration time to guarantee the same instance that was
	// registered with the tab manager is used here. Calling GetCustomWindowMenuCategory()
	// again could return a different object if the override doesn't cache its result.
	const TSharedPtr<FWorkspaceItem>* Cached = CachedModeWorkspaceCategories.Find(ApplicationMode->GetModeName());
	TSharedPtr<FWorkspaceItem> Category = (Cached && Cached->IsValid()) ? *Cached : DataViewsWorkspaceCategory;
	ensureMsgf(Category.IsValid(), TEXT("PushTabFactories called before any mode was registered. Tabs will have no workspace group in the Window menu"));

	for (auto FactoryIt = FactorySetToPush.CreateConstIterator(); FactoryIt; ++FactoryIt)
	{
		FTabSpawnerEntry& SpawnerEntry = FactoryIt.Value()->RegisterTabSpawner(TabManager.ToSharedRef(), ApplicationMode.Get());
		if (Category.IsValid())
		{
			SpawnerEntry.SetGroup(Category.ToSharedRef());
		}
	}
}

void FLiveLinkHubApplication::AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode)
{
	// Register tabs for a mode once it's registered so that other modes have access to that tab.
	Mode->RegisterTabFactoriesWithManager(TabManager);
	FWorkflowCentricApplication::AddApplicationMode(ModeName, MoveTemp(Mode));
}

void FLiveLinkHubApplication::SetCurrentMode(FName NewMode)
{
	// Behaves like FWorkflowCentricApplication, but does not clear or add tab spawners, allowing us to share tabs between modes.
	const bool bModeAlreadyActive = CurrentAppModePtr.IsValid() && (NewMode == CurrentAppModePtr->GetModeName());

	if (!bModeAlreadyActive)
	{
		check(TabManager.IsValid());

		const TSharedPtr<FApplicationMode> NewModePtr = GetApplicationModeList().FindRef(NewMode);

		LayoutExtenders.Reset();

		if (NewModePtr.IsValid())
		{
			if (NewModePtr->LayoutExtender.IsValid())
			{
				LayoutExtenders.Add(NewModePtr->LayoutExtender);
			}
			
			// Deactivate the old mode
			if (CurrentAppModePtr.IsValid())
			{
				CurrentAppModePtr->PreDeactivateMode();
				CurrentAppModePtr->DeactivateMode();
				RemoveToolbarExtender(CurrentAppModePtr->GetToolbarExtender());
				RemoveAllToolbarWidgets();
			}

			CurrentAppModePtr = NewModePtr;

			// Activate the new layout
			const TSharedRef<FTabManager::FLayout> NewLayout = CurrentAppModePtr->ActivateMode();
			RestoreFromLayout(NewLayout);

			// Give the new mode a chance to do init
			CurrentAppModePtr->PostActivateMode();

			AddToolbarExtender(NewModePtr->GetToolbarExtender());
			RegenerateMenusAndToolbars();
		}
	}

	AppModeChangedDelegate.Broadcast(NewMode);
}

void FLiveLinkHubApplication::PersistUserLayout(const FString& LayoutName, const TSharedPtr<FJsonObject>& JsonLayout)
{
	if (FString* LayoutPath = CachedLayouts.Find(LayoutName))
	{
		SaveLayoutToFile(*LayoutPath, JsonLayout);
	}
}

void FLiveLinkHubApplication::SaveLayoutAs()
{
	if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
	{
		const FString DefaultFile = LiveLinkHubLayoutFileUtils::LayoutDefaultFileName;

		TArray<FString> SaveFileNames;

		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const FString DefaultSaveDir = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_SAVE);

		const bool bFileSelected = DesktopPlatform->SaveFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("LiveLinkHubSaveAsTitle", "Save As").ToString(),
			DefaultSaveDir,
			DefaultFile,
			LiveLinkHubLayoutFileUtils::FileTypes,
			EFileDialogFlags::None,
			SaveFileNames);

		if (bFileSelected && SaveFileNames.Num() > 0)
		{
			const FString& SavePath = SaveFileNames[0];

			// Todo: Allow naming layouts differently from the filename? 
			FString LayoutName = FPaths::GetBaseFilename(SavePath);
			SaveLayoutToFile(SaveFileNames[0]);
		
			CachedLayouts.Add(LayoutName, SavePath);

			// Todo: We already have the json object in memory, we could fast track this instead of saving to disk then reloading
			RegisterUserLayout(LayoutName, SavePath);

			ULiveLinkHubUserSettings* Settings = GetMutableDefault<ULiveLinkHubUserSettings>();
			Settings->LayoutDirectories.AddUnique(FPaths::GetPath(SavePath));
			Settings->SaveConfig();

			SetCurrentMode(*LayoutName);
		}
	}
}

void FLiveLinkHubApplication::LoadLayout()
{
	const FString DefaultFile = LiveLinkHubLayoutFileUtils::LayoutDefaultFileName;

	TArray<FString> OpenFileNames;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

	const bool bFileSelected = DesktopPlatform->OpenFileDialog(
		ParentWindowWindowHandle,
		LOCTEXT("LiveLinkHubOpenTitle", "Open").ToString(),
		FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN),
		DefaultFile,
		LiveLinkHubLayoutFileUtils::LayoutDefaultFileName,
		EFileDialogFlags::None,
		OpenFileNames);

	if (bFileSelected && OpenFileNames.Num() > 0)
	{
		const FString LayoutPath = OpenFileNames[0];
		ULiveLinkHubUserSettings* Settings = GetMutableDefault<ULiveLinkHubUserSettings>();
		Settings->LayoutDirectories.AddUnique(FPaths::GetPath(LayoutPath));
		Settings->SaveConfig();

		FString LayoutName = FPaths::GetBaseFilename(LayoutPath);

		if (!CachedLayouts.Contains(LayoutName))
		{
			if (RegisterUserLayout(LayoutName, LayoutPath))
			{
				SetCurrentMode(*LayoutName);
			}
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not load layout %ls since it already exists.", *LayoutName);
		}
	}
}

void FLiveLinkHubApplication::ResetLayout()
{
	if (TSharedPtr<FLiveLinkHubApplicationMode> CurrentMode = StaticCastSharedPtr<FLiveLinkHubApplicationMode>(GetCurrentModePtr()))
	{
		constexpr bool bLoadUserLayout = false;
		RestoreFromLayout(CurrentMode->GetTabLayout().ToSharedRef(), bLoadUserLayout);
		AddToolbarExtender(CurrentMode->GetToolbarExtender());
		RegenerateMenusAndToolbars();
	}
}

TArray<FString> FLiveLinkHubApplication::GetUserLayouts()
{
	TArray<FString> Keys;
	CachedLayouts.GenerateKeyArray(Keys);
	Keys.Sort();
	return Keys;
}

void FLiveLinkHubApplication::DeleteUserLayout(const FString& LayoutName)
{
	if (FString* LayoutPath = CachedLayouts.Find(LayoutName))
	{
		IFileManager::Get().Delete(**LayoutPath);

		CachedLayouts.Remove(LayoutName);
		RemoveLiveLinkHubApplicationMode(*LayoutName);
	}
}

TOptional<FLiveLinkHubUserLayout> FLiveLinkHubApplication::ParseUserLayout(const FString& LayoutPath)
{
	TOptional<FLiveLinkHubUserLayout> UserLayout;

	FString Result;
	FFileHelper::LoadFileToString(Result, *LayoutPath);

	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(Result);

	TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
	if (FJsonSerializer::Deserialize(Reader, RootObject))
	{
		if (RootObject->HasField(TEXT("ParentMode")) && RootObject->HasField(TEXT("Layout")))
		{
			FLiveLinkHubUserLayout Layout = FLiveLinkHubUserLayout{ RootObject->GetField(TEXT("ParentMode"), EJson::String)->AsString(), RootObject->GetObjectField(TEXT("Layout")) };
			UserLayout = MoveTemp(Layout);
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Error, "Live Link Hub Layout missing ParentNode field.");
		}
	}
	else
	{
		UE_LOGF(LogLiveLinkHub, Error, "Failed to parse livelinkhub layout.");
	}

	return UserLayout;
}

TSharedPtr<FLiveLinkHubApplicationMode> FLiveLinkHubApplication::FindApplicationMode(const FString& ModeName)
{
	const TMap<FName, TSharedPtr<FApplicationMode>>& ModeList = GetApplicationModeList();
	ensureMsgf(CachedModeInfo.Contains(*ModeName), TEXT("Mode was not registered with LiveLinkHubApplication."));
	if (TSharedPtr<FApplicationMode> AppMode = ModeList.FindRef(*ModeName))
	{
		return StaticCastSharedPtr<FLiveLinkHubApplicationMode>(AppMode);
	}

	return nullptr;
}

void FLiveLinkHubApplication::SaveLayoutToFile(const FString& SavePath, TSharedPtr<FJsonObject> JsonLayout)
{
	if (SavePath.IsEmpty() && LastLayoutPath.IsEmpty())
	{
		return;
	}

	if (!SavePath.IsEmpty())
	{
		LastLayoutPath = SavePath;
	}

	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_SAVE, FPaths::GetPath(LastLayoutPath));
	FEditorDirectories::Get().SaveLastDirectories();

	if (!JsonLayout)
	{
		TSharedRef<FTabManager::FLayout> PersistentLayout = FLiveLinkHub::Get()->GetTabManager()->PersistLayout();
		JsonLayout = PersistentLayout->ToJson();
	}

	if (ensure(!LastLayoutPath.IsEmpty()))
	{
		TSharedRef<FJsonObject> LayoutJson = MakeShared<FJsonObject>();

		FString ParentModeName = GetCurrentMode().ToString();
		if (TSharedPtr<FLiveLinkHubApplicationMode> ParentMode = FindApplicationMode(ParentModeName))
		{
			if (ParentMode->IsUserLayout())
			{
				ParentModeName = StaticCastSharedPtr<FLiveLinkHubUserLayoutMode>(ParentMode)->GetParentMode()->GetModeName().ToString();
			}
		}
		LayoutJson->SetStringField(TEXT("ParentMode"), ParentModeName);
		LayoutJson->SetObjectField(TEXT("Layout"), JsonLayout);

		FString LayoutAsString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create( &LayoutAsString );
		if (!FJsonSerializer::Serialize(LayoutJson, Writer))
		{
			UE_LOGF(LogLiveLinkHub, Error, "Failed save layout as Json string");
			return;
		}

		FFileHelper::SaveStringToFile(LayoutAsString, *LastLayoutPath);
	}
}

TOptional<FLiveLinkHubAppModeInfo> FLiveLinkHubApplication::GetModeInfo(FName Mode) const
{
	TOptional<FLiveLinkHubAppModeInfo> ModeInfo;

	if (const FLiveLinkHubAppModeInfo* InfoPtr = CachedModeInfo.Find(Mode))
	{
		ModeInfo = *InfoPtr;
	}

	return ModeInfo;
}

FName FLiveLinkHubApplication::GetLayoutName(FName ModeName)
{
	FName Name = NAME_None;
	if (TSharedPtr<FApplicationMode> AppMode = GetApplicationModeList().FindRef(ModeName))
	{
		Name = StaticCastSharedPtr<FLiveLinkHubApplicationMode>(AppMode)->GetLayoutName();
	}

	return Name;
}

TArray<FName> FLiveLinkHubApplication::GetApplicationModes() const
{
	TArray<FName> ApplicationModeNames;
	CachedModeInfo.GenerateKeyArray(ApplicationModeNames);
	ApplicationModeNames.SetNum(Algo::RemoveIf(ApplicationModeNames, [this](FName Name) { return CachedLayouts.Contains(Name.ToString()); }));
	return ApplicationModeNames;
}

UToolMenu* FLiveLinkHubApplication::GenerateCommonActionsToolbar(FToolMenuContext& MenuContext)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ToolBarName = "AssetEditorToolbar.LiveLinkHub.CommonActions";

	UToolMenu* FoundMenu = ToolMenus->FindMenu(ToolBarName);

	if (!FoundMenu || !FoundMenu->IsRegistered())
	{
		FoundMenu = ToolMenus->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		FoundMenu->StyleName = "AssetEditorToolbar";

		FToolMenuSection& Section = FoundMenu->AddSection("CommonActions");

		Section.AddEntry(FToolMenuEntry::InitWidget("AppModeSwitcher", SNew(SApplicationModeSwitcher), LOCTEXT("AppModeSwitcherLabel", "ApplicationMode Switcher")));
		Section.AddSeparator(NAME_None);
	}

	return ToolMenus->GenerateMenu(ToolBarName, MenuContext);
}

void FLiveLinkHubApplication::RegisterFileMenu(UToolMenu* Menu)
{
	UToolMenu* FileMenu = Menu->AddSubMenu(
			"MainMenu",
			NAME_None,
			FLiveLinkHubApplicationMode::FileMenuExtensionPoint, // Prevents showing other submenus that would've been registered to File, which we don't care about.
			LOCTEXT("FileMenu", "File"),
			FText::GetEmpty()
		);
	
	// Explicitly bind LLHub's command list so these entries resolve correctly even when a sub-editor
	// (e.g. Media Profile) is focused and its own command list is in the active menu context.
	const TSharedPtr<const FUICommandList> HubCommands = FLiveLinkHub::Get()->GetCommandList();

	FToolMenuSection& FileAssetSection = FileMenu->AddSection("FileOpen", LOCTEXT("FileOpenHeading", "Open"), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	{
		FileAssetSection.AddMenuEntryWithCommandList(FLiveLinkHubCommands::Get().NewConfig, HubCommands);
		FileAssetSection.AddMenuEntryWithCommandList(FLiveLinkHubCommands::Get().OpenConfig, HubCommands);
		FileAssetSection.AddSubMenu("RecentConfigs", LOCTEXT("RecentConfigsSubmenu", "Open Recent Configs"), LOCTEXT("RestoreConfigToolTip", "Restore a config that was auto-saved."),
			FNewToolMenuChoice
			(
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
				{
					auto CreateMenuEntry = [](const FLiveLinkHubSessionFile& SessionFile)
					{
						TSharedPtr<ILiveLinkHubSessionManager> SessionManager = FLiveLinkHub::Get()->GetSessionManager();
							
						FToolUIAction ToolUIAction(
						FToolMenuExecuteAction::CreateLambda([SessionFile](const FToolMenuContext&)
						{
							FLiveLinkHub::Get()->GetSessionManager()->RestoreSession(SessionFile.FilePath);
						}));
						
						ToolUIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&)
						{
							return !FLiveLinkHub::Get()->GetPlaybackController()->IsInPlayback();
						});
							
						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							*SessionFile.FileName,
							FText::FromString(SessionFile.FileName),
							FText::FromString(SessionFile.LastModificationDate.ToString()),
							FSlateIcon(),
							ToolUIAction,
							EUserInterfaceActionType::None
						);

						return Entry;
					};

					// Recent config files that were saved/loaded.
					FToolMenuSection& RecentConfigsSection = ToolMenu->AddSection("RecentConfigs", LOCTEXT("RecentConfigsLabel", "Recent Configs"));

					TArray<FLiveLinkHubSessionFile> RecentFiles = GetMutableDefault<ULiveLinkHubUserSettings>()->GetRecentConfigFiles();

					for (const FLiveLinkHubSessionFile& File : RecentFiles)
					{
						RecentConfigsSection.AddEntry(CreateMenuEntry(File));
					}

					// Config files that originated from an autosave.
					FToolMenuSection& AutosavesSection = ToolMenu->AddSection("Autosaves", LOCTEXT("AutosaveLabel", "Autosaves"));

					TArray<FLiveLinkHubSessionFile> AutosaveFiles = FLiveLinkHub::Get()->GetAutosaveHandler()->GetAutosaveFiles();

					for (const FLiveLinkHubSessionFile& Autosave : AutosaveFiles)
					{
						AutosavesSection.AddEntry(CreateMenuEntry(Autosave));
					}
				})
			)
		);
	}

	FToolMenuSection& OpenAssetSection = FileMenu->AddSection("FileSave", LOCTEXT("FileSaveHeading", "Save"), FToolMenuInsert("FileOpen", EToolMenuInsertType::After));
	{
		OpenAssetSection.AddMenuEntryWithCommandList(FLiveLinkHubCommands::Get().SaveConfig, HubCommands);
		OpenAssetSection.AddMenuEntryWithCommandList(FLiveLinkHubCommands::Get().SaveConfigAs, HubCommands);
	}
}

void FLiveLinkHubApplication::RegisterMainMenu()
{
	static const FName MainMenuName("MainFrame.MainMenu");
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (ToolMenus->IsMenuRegistered(MainMenuName))
	{
		return;
	}

	UToolMenu* MenuBar = ToolMenus->RegisterMenu(MainMenuName, NAME_None, EMultiBoxType::MenuBar);
	MenuBar->StyleName = FName("WindowMenuBar");

	RegisterFileMenu(MenuBar);

	const bool bShowDevTools = FParse::Param(FCommandLine::Get(), TEXT("Development"))
		|| FEngineBuildSettings::IsInternalBuild();

	if (bShowDevTools)
	{
		UToolMenu* ToolsEntry = MenuBar->AddSubMenu(
			"MainMenu",
			NAME_None,
			"Tools",
			LOCTEXT("ToolsMenu", "Tools"),
			FText::GetEmpty()
		);

		ToolsEntry->AddDynamicSection(NAME_None, FNewSectionConstructChoice{FNewToolMenuDelegateLegacy::CreateRaw(this, &FLiveLinkHubApplication::CreateToolsMenu)});
	}

	UToolMenu* WindowEntry = MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Window",
		LOCTEXT("WindowMenu", "Window"),
		FText::GetEmpty()
	);

	UToolMenu* HelpEntry = MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Help",
		LOCTEXT("HelpMenu", "Help"),
		FText::GetEmpty()
	);

	FToolMenuEntry OpenLogsFolderEntry = FToolMenuEntry::InitMenuEntry(FLiveLinkHubCommands::Get().OpenLogsFolder, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen"));
	HelpEntry->AddMenuEntry(NAME_None, OpenLogsFolderEntry);
	FToolMenuEntry AboutMenuEntry = FToolMenuEntry::InitMenuEntry(FLiveLinkHubCommands::Get().OpenAboutMenu, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Help"));
	HelpEntry->AddMenuEntry(NAME_None, AboutMenuEntry);

	WindowEntry->AddDynamicSection(NAME_None, FNewSectionConstructChoice{FNewToolMenuDelegateLegacy::CreateRaw(this, &FLiveLinkHubApplication::FillWindowMenu)});
}

void FLiveLinkHubApplication::CreateToolsMenu(FMenuBuilder& MenuBuilder, UToolMenu*) const
{
#if !UE_BUILD_SHIPPING
	MenuBuilder.BeginSection("Development", LOCTEXT("DevelopmentHeader", "Development"));

#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToolsMenu_Development_AutomationTools", "Automation Tools"),
		FText(),
		FSlateIcon(),
		FExecuteAction::CreateStatic([]()
			{
				ISessionFrontendModule& SessionFrontend =
					FModuleManager::LoadModuleChecked<ISessionFrontendModule>("SessionFrontend");
				SessionFrontend.InvokeSessionFrontend(FName("AutomationPanel"));
			})
	);
#endif // #if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToolsMenu_Development_WidgetReflector", "Widget Reflector"),
		FText(),
		FSlateIcon(),
		FExecuteAction::CreateStatic([]()
			{
				ISlateReflectorModule& SlateReflector =
					FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector");
				SlateReflector.DisplayWidgetReflector();
			})
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToolsMenu_Development_MesageBusTester", "Message Bus Tester"),
		FText(),
		FSlateIcon(),
		FExecuteAction::CreateStatic([]()
			{
				IMessageBusTesterEditorModule& Module = FModuleManager::LoadModuleChecked<IMessageBusTesterEditorModule>("MessageBusTesterEditor");
				Module.DisplayMessageBusTester();
			})
	);

	MenuBuilder.EndSection();


#endif /*UE_BUILD_SHIPPING*/
}

void FLiveLinkHubApplication::FillWindowMenu(FMenuBuilder& MenuBuilder, UToolMenu*)
{
	GetTabManager()->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

UE::Editor::Toolbars::ECreateStatusBarOptions FLiveLinkHubApplication::GetStatusBarCreationOptions() const
{
	return UE::Editor::Toolbars::ECreateStatusBarOptions::HideContentBrowser | UE::Editor::Toolbars::ECreateStatusBarOptions::HideSourceControl;
}

TSharedPtr<SWidget> FLiveLinkHubApplication::CreateMenuBar(const TSharedPtr<FTabManager>& InTabManager, const FName InMenuName, FToolMenuContext& InToolMenuContext)
{
	RegisterMainMenu();

	InToolMenuContext.AppendCommandList(GetToolkitCommands());

	USlateTabManagerContext* ContextObject = NewObject<USlateTabManagerContext>();
	ContextObject->TabManager = TabManager;
	InToolMenuContext.AddObject(ContextObject);

	// Create the menu bar
	TSharedRef<SWidget> MenuBarWidget = UToolMenus::Get()->GenerateWidget(InMenuName, InToolMenuContext);
	if (MenuBarWidget != SNullWidget::NullWidget)
	{
		// Tell tab-manager about the multi-box for platforms with a global menu bar
		TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(MenuBarWidget);
		TabManager->SetMenuMultiBox(ConstCastSharedRef<FMultiBox>(MultiBoxWidget->GetMultiBox()), MultiBoxWidget);
	}

	return SNullWidget::NullWidget;
}

FText FLiveLinkHubApplication::GetToolkitName() const
{
	return LOCTEXT("LiveLinkHubLabel", "Live Link Hub");
}

FText FLiveLinkHubApplication::GetToolkitToolTipText() const
{
	return GetToolkitName();
}

void FLiveLinkHubApplication::DiscoverLayouts()
{
	TArray<FString> Files;

	for (const FString& Directory : GetLayoutDirectories())
	{
		IFileManager::Get().FindFiles(Files, *Directory, *LiveLinkHubLayoutFileUtils::LayoutFileExtension);

		for (const FString& File : Files)
		{
			FString LayoutName = File.LeftChop(LiveLinkHubLayoutFileUtils::LayoutFileExtension.Len() + 1);

			if (!CachedLayouts.Contains(LayoutName))
			{
				const FString LayoutFile = Directory / File;
				RegisterUserLayout(LayoutName, LayoutFile);
			}
		}
	}
}

bool FLiveLinkHubApplication::RegisterUserLayout(const FString& LayoutName, const FString& LayoutPath)
{
	if (TOptional<FLiveLinkHubUserLayout> UserLayout = ParseUserLayout(LayoutPath))
	{
		CachedLayouts.Add(LayoutName, LayoutPath);
		if (TSharedPtr<FLiveLinkHubApplicationMode> ParentMode = FindApplicationMode(UserLayout->ParentModeName))
		{
			TSharedRef<FLiveLinkHubUserLayoutMode> UserLayoutMode = MakeShared<FLiveLinkHubUserLayoutMode>(*LayoutName, UserLayout->JsonLayout.ToSharedRef(), ParentMode);
			AddLiveLinkHubApplicationMode(*LayoutName, UserLayoutMode);
		}
		return true;
	}

	return false;
}

TArray<FString> FLiveLinkHubApplication::GetLayoutDirectories() const
{
	FString UserSettingsDir = FLiveLinkHubApplication::UserSettingsDir() / TEXT("Layouts");
	FString LastSaveDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_SAVE);

	TArray<FString> AllPaths = { MoveTemp(UserSettingsDir), MoveTemp(LastSaveDirectory) };
	AllPaths.Append(GetDefault<ULiveLinkHubUserSettings>()->LayoutDirectories);

	return AllPaths;
}

#undef LOCTEXT_NAMESPACE /*LiveLinkHubApplication*/
