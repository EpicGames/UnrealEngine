// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubApplicationBase.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "Math/Color.h"
#include "Misc/Optional.h"

class FLiveLinkHubApplicationMode;

/** Cached information about a given application mode. Used to populate the mode switcher widget. */
struct FLiveLinkHubAppModeInfo
{
	/** Icon of the mode. */
	FSlateIcon Icon;
	/** Display name for the mode. */
	FText DisplayName;
	/** Whether this is a user layout mode. */
	bool bUserLayout = false;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnApplicationModeChanged, /*CurrentMode*/FName);

class FLiveLinkHubApplication : public FLiveLinkHubApplicationBase
{
public:
	FLiveLinkHubApplication();

	/** Get the delegate called when the app mode has changed. */
	FOnApplicationModeChanged& OnApplicationModeChanged() { return AppModeChangedDelegate; }

	/** Get information about a given mode. Returns an empty optional if the mode wasn't found or if it wasn't added to the app using AddLiveLinkHubApplicationMode. */
	TOptional<FLiveLinkHubAppModeInfo> GetModeInfo(FName ModeName) const;

	/** Get the name of the TabLayout for a given mode. */
	FName GetLayoutName(FName ModeName);

	/** Get the list of modes that were registered with the app. */
	TArray<FName> GetApplicationModes() const;

	/** Add a LiveLinkHub application mode. */
	virtual void AddLiveLinkHubApplicationMode(FName ModeName, TSharedRef<FLiveLinkHubApplicationMode> Mode);

	/** Push tab factories for the given mode. */
	virtual void PushTabFactories(const FWorkflowAllowedTabSet& FactorySetToPush, TSharedPtr<FLiveLinkHubApplicationMode> ApplicationMode);

	/** Save a user layout to disk. */
	void PersistUserLayout(const FString& LayoutName, const TSharedPtr<FJsonObject>& JsonLayout);

	/** Prompt the user to save the current layout file to disk. */
	void SaveLayoutAs();

	/**  Prompt the user to load a layout file from disk.*/
	void LoadLayout();

	/** Reset the current layout to its code-defined version. (Only available for non-user layouts) */
	void ResetLayout();

	/** Get the list of user layouts. */
	TArray<FString> GetUserLayouts();

	/** Delete a user layout. This will also delete the file on disk. */
	void DeleteUserLayout(const FString& LayoutName);

	//~ Begin FWorkflowCentricApplication interface
	virtual void SetCurrentMode(FName NewMode) override;
	//~ End FWorkflowCentricApplication interface

protected:
	/** Get the path to the user settings. */
	static FString UserSettingsDir()
	{
		static const FString UserDir = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("LiveLinkHub"));
		return UserDir;
	}

	//~ Begin FWorkflowCentricApplication interface
	virtual FName GetToolkitFName() const override { return "LiveLinkHub"; }
	virtual FText GetBaseToolkitName() const override { return INVTEXT("LiveLinkHub Editor"); }
	virtual FString GetWorldCentricTabPrefix() const override { return TEXT("LiveLinkHub"); }
	FLinearColor GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}
	//~ End FWorkflowCentricApplication interface

	//~ Begin AssetEditorToolkit interface
	virtual UToolMenu* GenerateCommonActionsToolbar(FToolMenuContext& MenuContext) override;
	virtual UE::Editor::Toolbars::ECreateStatusBarOptions GetStatusBarCreationOptions() const override;
	virtual TSharedPtr<SWidget> CreateMenuBar(const TSharedPtr<FTabManager>& InTabManager, const FName InMenuName, FToolMenuContext& InToolMenuContext) override;
	virtual void MapToolkitCommands() {}
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FName GetOverrideTabId() const override { return "LiveLinkHub"; }
	//~ End AssetEditorToolkit interface

	/** Iterate through layout directories to find .llhlayout files. */
	void DiscoverLayouts();

	/** Creates a user layout app mode from a layout file. */
	bool RegisterUserLayout(const FString& LayoutName, const FString& LayoutFile);

private:
	/** Don't allow LiveLinkHub to call this directly since it wouldn't register the cached mode info. */
	virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode) override;

	/** Register and populate the file menu with the config commands. */
	void RegisterFileMenu(UToolMenu* Menu);
	/** Create the main menu bar. */
	void RegisterMainMenu();
	/** When in dev mode, creates the tool menu that contains the widget reflector and automation window. */
	void CreateToolsMenu(FMenuBuilder& MenuBuilder, UToolMenu*) const;
	void FillWindowMenu(FMenuBuilder& MenuBuilder, UToolMenu*);

	/** Used to populate the status toolbar with widgets of the current mode. */
	void AddToolbarExtenders(FToolBarBuilder&);

	/** Persist layout and save it to the specified path. */
	void SaveLayoutToFile(const FString& SavePath = TEXT(""), TSharedPtr<FJsonObject> JsonLayout = TSharedPtr<FJsonObject>{});

	TArray<FString> GetLayoutDirectories() const;

	/** Read the layout file and convert it to a JsonObject. */
	TOptional<struct FLiveLinkHubUserLayout> ParseUserLayout(const FString& LayoutName);

	/** Get an application mode from  */
	TSharedPtr<FLiveLinkHubApplicationMode> FindApplicationMode(const FString& ModeName);

	void RemoveLiveLinkHubApplicationMode(FName ModeName);

private:
	/** Delegate called when the active app mode has changed. */
	FOnApplicationModeChanged AppModeChangedDelegate;

	/** Map of mode name to cached mode info. */
	TMap<FName, FLiveLinkHubAppModeInfo> CachedModeInfo;

	/** Shared workspace menu category for all tabs in the Window menu. */
	TSharedPtr<FWorkspaceItem> DataViewsWorkspaceCategory;

	/** Workspace category resolved at registration time for each mode, keyed by mode name. */
	TMap<FName, TSharedPtr<FWorkspaceItem>> CachedModeWorkspaceCategories;

	/** Path to the last saved layout. */
	FString LastLayoutPath;

	/** Layout name to file path */
	TMap<FString, FString> CachedLayouts;
};
