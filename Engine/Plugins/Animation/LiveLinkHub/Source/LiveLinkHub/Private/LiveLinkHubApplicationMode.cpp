// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubApplicationMode.h"

#include "LiveLinkHubApplicationBase.h"
#include "LiveLinkHubModule.h"
#include "Recording/LiveLinkHubRecordingController.h"
#include "UI/Widgets/SLiveLinkHubSettings.h"
#include "UI/Widgets/SLiveLinkHubTopologyModeSwitcher.h"
#include "UI/Widgets/SLiveLinkRecordingSessionInfo.h"
#include "UI/Widgets/SLiveLinkTimecode.h"
#include "ToolMenus.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

const FName ILiveLinkHubApplicationModeFactory::ModularFeatureName = "LiveLinkHubApplicationModeFactory";
const FName FLiveLinkHubApplicationMode::FileMenuExtensionPoint = "LiveLinkHubFileMenu";


FLiveLinkHubApplicationMode::FLiveLinkHubApplicationMode(FName InApplicationMode, FText InDisplayName, TSharedPtr<FLiveLinkHubApplicationBase> InApp)
	: FApplicationMode(InApplicationMode)
{
	LayoutIni = TEXT("LiveLinkHubLayout");
	WeakApp = InApp;
	DisplayName = InDisplayName;

	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(InDisplayName);
}

TArray<TSharedRef<SWidget>> FLiveLinkHubApplicationMode::GetToolbarWidgets()
{
	TArray<TSharedRef<SWidget>> Widgets = GetToolbarWidgets_Impl();

	// LiveLinkHub Settings should always be last.
	Widgets.Add(SNew(SLiveLinkHubSettings));

	return Widgets;
}

void FLiveLinkHubApplicationMode::PostActivateMode()
{
	FToolMenuOwnerScoped OwnerScoped(GetModeName());

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("LiveLinkHub.ToolBar"));

	FToolMenuSection& GlobalWidgetsSection = Menu->AddSection(TEXT("GlobalWidgets"), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	TSharedRef<SWidget> GlobalWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0, 0.0)
		.VAlign(VAlign_Center)
		[
			SNew(SLiveLinkHubTopologyModeSwitcher)
		];

	GlobalWidgetsSection.AddEntry(
		FToolMenuEntry::InitWidget(TEXT("TestWidget"), GlobalWidget, FText::GetEmpty(), true, false)
	);

	GlobalWidgetsSection.AddSeparator(NAME_None);

	FToolMenuSection& SessionConfigSection = Menu->AddSection(TEXT("ModeWidgets"), FText::GetEmpty(), FToolMenuInsert(TEXT("GlobalWidgets"), EToolMenuInsertType::After));

	TSharedRef<SHorizontalBox> ModeWidgets = SNew(SHorizontalBox);

	TArray<TSharedRef<SWidget>> Widgets = GetStatusBarWidgets_Impl();
	for (const TSharedRef<SWidget>& Widget : Widgets)
	{
		ModeWidgets->AddSlot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.0, 0.0, 1.0))
			.VAlign(VAlign_Center)
			[
				Widget
			];
	}

	SessionConfigSection.AddEntry(
		FToolMenuEntry::InitWidget(TEXT("ModeWidget"), ModeWidgets, FText::GetEmpty(), true, false)
	);

	// Recording toolbar - available across all modes.
	UToolMenu* Toolbar = UToolMenus::Get()->ExtendMenu(TEXT("AssetEditor.LiveLinkHub.ToolBar"));

	FToolMenuSection& RecordingSessionSection = Toolbar->AddSection("SlateWidgets");
	RecordingSessionSection.AddEntry(
		FToolMenuEntry::InitWidget("SlateWidget", SNew(SLiveLinkRecordingSessionInfo), FText::GetEmpty(), true, false)
	);
	RecordingSessionSection.Alignment = EToolMenuSectionAlign::Last;

	FToolMenuSection& TimecodeSection = Toolbar->AddSection("TimecodeWidgets", FText::GetEmpty(), FToolMenuInsert{ "SlateWidgets", EToolMenuInsertType::After });
	TimecodeSection.Alignment = EToolMenuSectionAlign::Last;

	FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
	TimecodeSection.AddEntry(
		FToolMenuEntry::InitWidget("RecordButton", LiveLinkHubModule.GetRecordingController()->MakeRecordToolbarEntry(), FText::GetEmpty(), true, false)
	);

	TimecodeSection.AddEntry(
		FToolMenuEntry::InitWidget("Timecode", SNew(SLiveLinkTimecode), FText::GetEmpty(), true, false)
	);

	Toolbar->AddSection("MediaProfile", FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
}

void FLiveLinkHubApplicationMode::PreDeactivateMode()
{
	UToolMenus::Get()->UnregisterOwner(GetModeName());
}

