// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubClient.h"

#include "Clients/LiveLinkHubProvider.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkLog.h"
#include "LiveLinkRoleTrait.h"
#include "LiveLinkSourceCollection.h"
#include "LiveLinkSubject.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Recording/LiveLinkHubPlaybackSourceSettings.h"
#include "Stats/Stats.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"

DECLARE_CYCLE_STAT(TEXT("LiveLinkHub - Push StaticData"), STAT_LiveLinkHub_PushStaticData, STATGROUP_LiveLinkHub);
DECLARE_CYCLE_STAT(TEXT("LiveLinkHub - Push FrameData"), STAT_LiveLinkHub_PushFrameData, STATGROUP_LiveLinkHub);

#define LOCTEXT_NAMESPACE "LiveLinkHub.LiveLinkHubClient"

FLiveLinkHubClient::~FLiveLinkHubClient()
{
	constexpr bool bUseUnmappedData = true;
	UnregisterGlobalSubjectFramesDelegate(StaticDataAddedHandle, FrameDataAddedHandle, bUseUnmappedData);
}

void FLiveLinkHubClient::CacheSubjectSettings(const FLiveLinkSubjectKey& SubjectKey, ULiveLinkSubjectSettings* Settings) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectKey))
	{
		ULiveLinkSourceSettings* SourceSettings = GetSourceSettings(SubjectKey.Source);
		ULiveLinkSubjectSettings* SubjectSettings = Settings;

		SubjectItem->GetLiveSubject()->CacheSettings(SourceSettings, SubjectSettings);
		SubjectItem->GetLiveSubject()->SetStaticDataAsRebroadcasted(false);
		BroadcastStaticDataUpdate(SubjectItem->GetLiveSubject(), SubjectItem->GetSubject()->GetRole(), SubjectItem->GetLiveSubject()->GetStaticData());
	}
}

bool FLiveLinkHubClient::CreateSource(const FLiveLinkSourcePreset& InSourcePreset)
{
	// Create fake sources if we're in playback mode so we don't process livelink data while we're playing a recording.
	if (LiveLinkHub.Pin()->IsInPlayback())
	{
		TStrongObjectPtr<ULiveLinkSourceSettings> PlaybackSourceSettings = TStrongObjectPtr<ULiveLinkSourceSettings>(NewObject<ULiveLinkSourceSettings>(GetTransientPackage(), ULiveLinkHubPlaybackSourceSettings::StaticClass()));

		FLiveLinkSourcePreset ModifiedPreset = InSourcePreset;
		ModifiedPreset.Settings = PlaybackSourceSettings.Get();

		// Override the incoming source settings to create a playback source instead.
		if (InSourcePreset.Settings && InSourcePreset.Settings->Factory)
		{
			PlaybackSourceSettings->ConnectionString = InSourcePreset.Settings->Factory.GetDefaultObject()->GetSourceDisplayName().ToString();
		}

		return FLiveLinkClient::CreateSource(ModifiedPreset);
	}
	else
	{
		return FLiveLinkClient::CreateSource(InSourcePreset);
	}
}

FText FLiveLinkHubClient::GetSourceStatus(FGuid InEntryGuid) const
{
	if (LiveLinkHub.Pin()->IsInPlayback())
	{
		return LOCTEXT("PlaybackText", "Playback");
	}

	return FLiveLinkClient::GetSourceStatus(InEntryGuid);
}

void FLiveLinkHubClient::RemoveSubject_AnyThread(const FLiveLinkSubjectKey& InSubjectKey)
{
	OnSubjectMarkedPendingKill_AnyThread().Broadcast(InSubjectKey);

	FLiveLinkClient::RemoveSubject_AnyThread(InSubjectKey);
}

bool FLiveLinkHubClient::AddVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey, TSubclassOf<ULiveLinkVirtualSubject> VirtualSubjectClass)
{
	bool bResult =  FLiveLinkClient::AddVirtualSubject(VirtualSubjectKey, VirtualSubjectClass);
	if (bResult)
	{
		bVirtualSubjectsPresent = true;
	}

	return bResult;
}

void FLiveLinkHubClient::RemoveVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey)
{
	FLiveLinkClient::RemoveVirtualSubject(VirtualSubjectKey);

	bool bAnyVirtualSubject = false;
	Collection->ForEachSubject([&bAnyVirtualSubject](const FLiveLinkCollectionSourceItem& SourceItem, const FLiveLinkCollectionSubjectItem& SubjectItem)
	{
		bAnyVirtualSubject = bAnyVirtualSubject || !!SubjectItem.GetVirtualSubject();;
	});

	bVirtualSubjectsPresent = bAnyVirtualSubject;
}

TSharedPtr<ILiveLinkProvider> FLiveLinkHubClient::GetRebroadcastLiveLinkProvider() const
{
	return FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkProvider();
}

void FLiveLinkHubClient::BroadcastStaticDataUpdate(FLiveLinkSubject* InLiveSubject, TSubclassOf<ULiveLinkRole> InRole, const FLiveLinkStaticDataStruct& InStaticData) const
{
	OnStaticDataReceivedDelegate_AnyThread.Broadcast(InLiveSubject->GetSubjectKey(), InRole, InStaticData);
}

void FLiveLinkHubClient::OnStaticDataAdded(FLiveLinkSubjectKey SubjectKey,  TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkStaticDataStruct& InStaticData)
{
	OnStaticDataReceivedDelegate_AnyThread.Broadcast(SubjectKey, SubjectRole, InStaticData);
}

void FLiveLinkHubClient::OnFrameDataAdded(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkFrameDataStruct& InFrameData)
{
	OnFrameDataReceivedDelegate_AnyThread.Broadcast(InSubjectKey, InFrameData);
}


#undef LOCTEXT_NAMESPACE
