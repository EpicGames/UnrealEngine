// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkClient.h"

#include "Delegates/DelegateCombinations.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

class FLiveLinkHubPlaybackController;
class FLiveLinkHubRecordingController;
class FLiveLinkSubject;
class ILiveLinkHub;
struct ILiveLinkProvider;

DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnFrameDataReceived_AnyThread, const FLiveLinkSubjectKey&, const FLiveLinkFrameDataStruct&);
DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnStaticDataReceived_AnyThread, const FLiveLinkSubjectKey&, TSubclassOf<ULiveLinkRole>, const FLiveLinkStaticDataStruct&);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSubjectMarkedPendingKill_AnyThread, const FLiveLinkSubjectKey&);


class FLiveLinkHubClient : public FLiveLinkClient
{
public:
	FLiveLinkHubClient(TSharedPtr<ILiveLinkHub> InLiveLinkHub, FTSSimpleMulticastDelegate& InTickingDelegate)
		: FLiveLinkClient(InTickingDelegate)
		, LiveLinkHub(MoveTemp(InLiveLinkHub))
	{
		constexpr bool bUseUnmappedData = true;
		RegisterGlobalSubjectFramesDelegate(FOnLiveLinkSubjectStaticDataAdded::FDelegate::CreateRaw(this, &FLiveLinkHubClient::OnStaticDataAdded),
			FOnLiveLinkSubjectFrameDataAdded::FDelegate::CreateRaw(this, &FLiveLinkHubClient::OnFrameDataAdded),
			StaticDataAddedHandle, FrameDataAddedHandle, bUseUnmappedData);
	}

	FLiveLinkHubClient(TSharedPtr<ILiveLinkHub> InLiveLinkHub)
		: FLiveLinkClient()
		, LiveLinkHub(MoveTemp(InLiveLinkHub))
	{
		constexpr bool bUseUnmappedData = true;
		RegisterGlobalSubjectFramesDelegate(FOnLiveLinkSubjectStaticDataAdded::FDelegate::CreateRaw(this, &FLiveLinkHubClient::OnStaticDataAdded),
			FOnLiveLinkSubjectFrameDataAdded::FDelegate::CreateRaw(this, &FLiveLinkHubClient::OnFrameDataAdded),
			StaticDataAddedHandle, FrameDataAddedHandle, bUseUnmappedData);
	}

	virtual ~FLiveLinkHubClient();
	
	/** Get the delegate called when frame data is received. */
	FOnFrameDataReceived_AnyThread& OnFrameDataReceived_AnyThread()
	{
		return OnFrameDataReceivedDelegate_AnyThread;
	}

	/** Get the delegate called when static data is received. */
	FOnStaticDataReceived_AnyThread& OnStaticDataReceived_AnyThread()
	{
		return OnStaticDataReceivedDelegate_AnyThread;
	}

	/** 
	 * Get the delegate called when a subject is marked for deletion. 
	 * This delegate will fire as soon as the subject is marked for deletion, while the OnSubjectRemovedDelegate may trigger at a later time.
	 */
	FOnSubjectMarkedPendingKill_AnyThread& OnSubjectMarkedPendingKill_AnyThread()
	{
		return OnSubjectMarkedPendingKillDelegate_AnyThread;
	}

	/** Cache subject settings for the subject specified by the subject key. */
	virtual void CacheSubjectSettings(const FLiveLinkSubjectKey& SubjectKey, ULiveLinkSubjectSettings* Settings) const override;

public:
	//~ Begin ILiveLinkClient interface
	virtual bool CreateSource(const FLiveLinkSourcePreset& InSourcePreset) override;
	virtual FText GetSourceStatus(FGuid InEntryGuid) const override;
	virtual void RemoveSubject_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) override;
	virtual bool AddVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey, TSubclassOf<ULiveLinkVirtualSubject> VirtualSubjectClass) override;
    virtual void RemoveVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey) override;
	//~ End ILiveLinkClient interface

	//~ Begin FLiveLinkClient interface
	virtual TSharedPtr<ILiveLinkProvider> GetRebroadcastLiveLinkProvider() const override;
	//~ End FLiveLinkClient interface

private:
	/** Broadcast a static data update to this client's listeners. */
	void BroadcastStaticDataUpdate(FLiveLinkSubject* InLiveSubject, TSubclassOf<ULiveLinkRole> InRole, const FLiveLinkStaticDataStruct& InStaticData) const;

	//~ Delegates called by LiveLinkClient
	void OnStaticDataAdded(FLiveLinkSubjectKey SubjectKey, TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkStaticDataStruct& InStaticData);
	void OnFrameDataAdded(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkFrameDataStruct& InFrameData);

private:
	/** Weak pointer to the live link hub. */
	TWeakPtr<ILiveLinkHub> LiveLinkHub;
    /** Delegate called when frame data is received. */
    FOnFrameDataReceived_AnyThread OnFrameDataReceivedDelegate_AnyThread;
    /** Delegate called when static data is received. */
    FOnStaticDataReceived_AnyThread OnStaticDataReceivedDelegate_AnyThread;
	/** Delegate called when a subject is marked for deletion. */
	FOnSubjectMarkedPendingKill_AnyThread OnSubjectMarkedPendingKillDelegate_AnyThread;
	/** Whether there are virtual subjects at the moment. Used to determine if we should cache frame data for their usage. */
	std::atomic<bool> bVirtualSubjectsPresent = false;

	//~ Delegate handles given by FLiveLinkClient.
	FDelegateHandle StaticDataAddedHandle;
	FDelegateHandle FrameDataAddedHandle;
};
