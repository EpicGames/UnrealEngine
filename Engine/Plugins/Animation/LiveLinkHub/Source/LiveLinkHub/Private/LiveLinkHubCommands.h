// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubCommands"

/**
 * The Live Link Hub commands.
 */
class FLiveLinkHubCommands
	: public TCommands<FLiveLinkHubCommands>
{
public:

	/** Default constructor. */
	FLiveLinkHubCommands()
		: TCommands<FLiveLinkHubCommands>(
			"LiveLinkHub",
			NSLOCTEXT("Contexts", "LiveLinkHub", "Live Link Hub"),
			NAME_None, FAppStyle::GetAppStyleSetName()
		)
	{ }

public:

	//~ TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(NewConfig, "New Config", "Create a new config", EUserInterfaceActionType::Button, FInputChord(EKeys::N, EModifierKey::Control));
		UI_COMMAND(OpenConfig, "Open Config...", "Open an existing config", EUserInterfaceActionType::Button, FInputChord(EKeys::O, EModifierKey::Control));
		UI_COMMAND(SaveConfigAs, "Save Config As...", "Save a config to a new file", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::S));
		UI_COMMAND(SaveConfig, "Save Config", "Save the current config", EUserInterfaceActionType::Button, FInputChord(EKeys::S, EModifierKey::Control));
		UI_COMMAND(OpenLogsFolder, "Open Logs Folder", "Open the folder containing the logs.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(OpenAboutMenu, "About", "Open the About Live Link Hub window.", EUserInterfaceActionType::Button, FInputChord());
	}

public:
	/** Start a new config. */
	TSharedPtr<FUICommandInfo> NewConfig;
	/** Open an existing config. */
	TSharedPtr<FUICommandInfo> OpenConfig;
	/** Save the current configuration to a new file. */
	TSharedPtr<FUICommandInfo> SaveConfigAs;
	/** Save the current configuration to an existing file. */
	TSharedPtr<FUICommandInfo> SaveConfig;
	/** Open the folder containing the logs for live link hub. */
	TSharedPtr<FUICommandInfo> OpenLogsFolder;
	/** Open the About window. */
	TSharedPtr<FUICommandInfo> OpenAboutMenu;
};

#undef LOCTEXT_NAMESPACE