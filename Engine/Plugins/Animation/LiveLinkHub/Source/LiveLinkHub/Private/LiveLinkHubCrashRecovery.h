// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Session/LiveLinkHubAutosaveHandler.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubCrashRecovery"

/**
 * Utility class that allows recovering the last autosave file if it detects that the app did not close correctly in its last run.
 */
class FLiveLinkHubCrashRecovery
{
public:
	FLiveLinkHubCrashRecovery()
	{
		// If we recover, delete the file
		if (IFileManager::Get().FileExists(*CanaryFilePath))
		{
			UE_LOGF(LogLiveLinkHub, Display, "Detected LiveLinkHub canary file.");
			ReadCanaryFile();
		}
		else
		{
			// Only create a canary file for the first LLH instance.
			CreateCanaryFile();
		}
	}

	~FLiveLinkHubCrashRecovery()
	{
		if (bCreatedCanaryFile)
		{
			DeleteCanaryFile();
		}
	}

private:
	/** Read the canary file from disk and prompt the user for recovering from the last autosave. */
	void ReadCanaryFile()
	{
		TArray<uint8> FileMemory;
		FFileHelper::LoadFileToArray(FileMemory, *CanaryFilePath);

		FMemoryReader Reader{ FileMemory };
		TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(&Reader);
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			uint32 PID = 0;
			FString AppName;
			if (JsonObject->TryGetNumberField(TEXT("PID"), PID) && JsonObject->TryGetStringField(TEXT("ApplicationName"), AppName))
			{
				// Make sure we don't accidentally detect a crash when the process is still open.

				FProcHandle Handle = FPlatformProcess::OpenProcess(PID);
				ON_SCOPE_EXIT{ FPlatformProcess::CloseProc(Handle); };

				bool bIsProcRunning = FPlatformProcess::IsProcRunning(Handle);
				bool bHandleRecycled = bIsProcRunning && AppName != FPlatformProcess::GetApplicationName(PID);

				if (!bIsProcRunning || bHandleRecycled)
				{
					const bool bRecoverInDev = GConfig->GetBoolOrDefault(TEXT("LiveLinkHub"), TEXT("bRecoverCrashInDev"), false, GEngineIni);
					if (!FPlatformMisc::IsDebuggerPresent() || bRecoverInDev)
					{
						PromptRecoverLastSave(PID);
					}

					DeleteCanaryFile();

					// Make sure to recreate canary file if we've deleted it.
					CreateCanaryFile();
				}
			}
		}
	}

	/** Create a canary file that contains a PID in case another LiveLinkHub instance launches. */
	void CreateCanaryFile()
	{
		const TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*CanaryFilePath));
		if (Ar)
		{
			TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			uint32 CurrentProcessId = FPlatformProcess::GetCurrentProcessId();

			JsonObject->SetNumberField(TEXT("PID"), FPlatformProcess::GetCurrentProcessId());
			const FString ApplicationName = FPlatformProcess::GetApplicationName(CurrentProcessId);

			JsonObject->SetStringField(TEXT("ApplicationName"), ApplicationName);

			const TSharedRef<TJsonWriter<UTF8CHAR>> JsonWriter = TJsonWriterFactory<UTF8CHAR>::Create(Ar.Get(), 0);
			FJsonSerializer::Serialize(JsonObject, JsonWriter);

			ensure(Ar->Close());

			bCreatedCanaryFile = true;
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not read canary file (%ls) when attempting to recover from a crash.", *CanaryFilePath);
		}
	}

	/** Delete canary file on exit. */
	void DeleteCanaryFile()
	{
		IFileManager::Get().Delete(*CanaryFilePath);
	}

	/** Prompt user to recover the last save file. */
	void PromptRecoverLastSave(uint32 ProcessId)
	{
		if (TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow())
		{
			/**
			 * Not sure if it's a combination of it being minimized + startup screen but I could not get the Recovery window to show up without HACK_ForceToFront.
			 * Things that did not work:
			 * - Adding a timer for tick and calling BringToFront there
			 * - Calling Maximize on the root window
			 * - Calling BringToFront with bForce=true.
			 * 
			 * So I instead copied the logic from CrashReportClient.
			 */
			const bool bForceBringToFront = (false || (PLATFORM_MAC));

			ParentWindow->HACK_ForceToFront();
			ParentWindow->BringToFront(bForceBringToFront);
		}

		const EAppReturnType::Type Response = FMessageDialog::Open(
			EAppMsgCategory::Info,
			EAppMsgType::YesNo,
			LOCTEXT("RecoverSave", "The last instance of Live Link Hub did not close correctly.\nDo you want to recover the last save file?"),
			LOCTEXT("RecoverSaveTitle", "Live Link Hub closed unexpectedly.")
		);

		const bool bRecoverLastSave = (Response == EAppReturnType::Yes);
		if (bRecoverLastSave)
		{
			RestoreLastAutosave(ProcessId);
			UE_LOGF(LogLiveLinkHub, Display, "Recovering last autosave.");
		}
	}

	/** Restore the last autosave file that's compatible with the canary file. */
	void RestoreLastAutosave(uint32 CompatibleProcessId)
	{
		TArray<FLiveLinkHubSessionFile> SaveFiles = FLiveLinkHub::Get()->GetAutosaveHandler()->GetAutosaveFiles();

		for (const FLiveLinkHubSessionFile& File : SaveFiles)
		{
			if (TStrongObjectPtr<ULiveLinkHubSessionData> SessionDataPtr = FLiveLinkHub::Get()->GetSessionManager()->ReadSessionFile(File.FilePath))
			{
				// Since this object lives in the Transient package.
				// Only try matching a save file with a process Id that matches the canary file.
				if (SessionDataPtr->ProcessId == CompatibleProcessId)
				{
					UE_LOGF(LogLiveLinkHub, Display, "Restoring session from save %ls", *File.FileName);
					FLiveLinkHub::Get()->GetSessionManager()->RestoreSession(SessionDataPtr.Get());
					return;
				}
			}
		}

		UE_LOGF(LogLiveLinkHub, Display, "Could not find a valid autosave file to recover from crash.");
	}
	

private:
	/** Default canary file path. */
	const FString CanaryFilePath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("LiveLinkHub"), TEXT("LiveLinkHub.canary"));

	/** Whether this instance created a canary file. (If a LiveLinkHub instance is launched while another one is running, we don't create one). */
	bool bCreatedCanaryFile = false;
};


#undef LOCTEXT_NAMESPACE
