// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Clients/LiveLinkHubClientsController.h"
#include "Features/IModularFeatures.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "ILiveLinkClient.h"
#include "LiveLinkHubApplicationBase.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkClientPanelToolbar.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubApplicationMode.h"
#include "LiveLinkHubCommands.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkPanelController.h"
#include "LiveLinkTypes.h"
#include "Modules/ModuleManager.h"
#include "Recording/LiveLinkHubRecordingListController.h"
#include "Subjects/LiveLinkHubSubjectController.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UI/Widgets/SLiveLinkHubSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubCreatorMode"

struct FLiveLinkHubDetailsTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubDetailsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedPtr<FLiveLinkPanelController>& InPanelController)
		: FWorkflowTabFactory(UE::LiveLinkHub::SourceDetailsTabId, MoveTemp(InHostingApp))
		, WeakPanelController(InPanelController)
	{
		TabLabel = LOCTEXT("DetailsTabLabel", "Details");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		TSharedRef<SWidget> DetailsWidget = SNullWidget::NullWidget;
		if (TSharedPtr<FLiveLinkPanelController> PanelController = WeakPanelController.Pin())
		{
			DetailsWidget = PanelController->GetCombinedDetailsWidget();
		}

		return DetailsWidget;
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("SourceDetailisTabToolTip", "Displays details for the selected LiveLink source.");
	}

private:
	TWeakPtr<FLiveLinkPanelController> WeakPanelController;
};

struct FLiveLinkHubDataDevicesTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubDataDevicesTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedPtr<FLiveLinkPanelController>& InPanelController)
		: FWorkflowTabFactory(UE::LiveLinkHub::SubjectsTabId, MoveTemp(InHostingApp))
		, WeakPanelController(InPanelController)
	{
		TabLabel = LOCTEXT("DataDevicesLabel", "Data Devices");
		TabIcon = FSlateIcon("LiveLinkStyle", "LiveLinkHub.Subjects.Icon");

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		TSharedRef<SWidget> SubjectsView = SNullWidget::NullWidget;
		if (TSharedPtr<FLiveLinkPanelController> PanelController = WeakPanelController.Pin())
		{
			SubjectsView = PanelController->SubjectsView->GetWidget();
			return SubjectsView;
		}
		return SNullWidget::NullWidget;
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("DataDevicesTabToolTip", "View the list of Live Link subjects, sources, and devices.");
	}

private:
	/** Panel controller used to get the list of data devices. */
	TWeakPtr<FLiveLinkPanelController> WeakPanelController;
};

struct FLiveLinkHubRecordingListTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubRecordingListTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(UE::LiveLinkHub::RecordingListTabId, MoveTemp(InHostingApp))
	{
		TabLabel = LOCTEXT("RecordingListTabLabel", "Recordings List");
		TabIcon = FSlateIcon(UE::LiveLinkHub::LiveLinkStyleName, TEXT("LiveLinkHub.Playback.Icon"));

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return FLiveLinkHub::Get()->GetRecordingListController()->MakeRecordingList();
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("RecordingListTabToolTip", "Hosts the LiveLink recordings list.");
	}
};

struct FLiveLinkHubPlaybackTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubPlaybackTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(UE::LiveLinkHub::PlaybackTabId, MoveTemp(InHostingApp))
	{
		TabLabel = LOCTEXT("PlaybackTabLabel", "Playback");
		TabIcon = FSlateIcon(UE::LiveLinkHub::LiveLinkStyleName, TEXT("LiveLinkHub.Playback.Icon"));
		bShouldAutosize = true;

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return FLiveLinkHub::Get()->GetPlaybackController()->GetPlaybackWidget();
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("PlaybackTabToolTip", "Hosts the LiveLink recording playback functionality.");
	}
};

struct FLiveLinkHubClientsTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubClientsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(UE::LiveLinkHub::ClientsTabId, MoveTemp(InHostingApp))
	{
		TabLabel = LOCTEXT("ClientsTabLabel", "Clients");
		TabIcon = FSlateIcon("LiveLinkStyle", "LiveLinkHub.Clients.Icon");

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		TSharedPtr<FLiveLinkHubClientsController> ClientsController = LiveLinkHubModule.GetLiveLinkHub()->GetClientsController();

		return ClientsController->MakeClientsView();
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("ClientsTabToolTip", "Displays the list of connected Unreal Editor instances.");
	}
};

struct FLiveLinkHubClientDetailsTabSummoner : public FWorkflowTabFactory
{
public:
	FLiveLinkHubClientDetailsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(UE::LiveLinkHub::ClientDetailsTabId, MoveTemp(InHostingApp))
	{
		TabLabel = LOCTEXT("ClientDetailsTabLabel", "Client Details");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		TSharedPtr<FLiveLinkHubClientsController> ClientsController = LiveLinkHubModule.GetLiveLinkHub()->GetClientsController();

		return ClientsController->MakeClientDetailsView();
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("ClientDetailsTabToolTip", "Displays details for the selected client.");
	}
};

/** Default application mode for the hub. Hosts the tabs necessary for viewing Sources, Subjects and clients. */
class FLiveLinkHubCreatorAppMode : public FLiveLinkHubApplicationMode
{
public:
	FLiveLinkHubCreatorAppMode(TSharedPtr<FLiveLinkHubApplicationBase> App)
		: FLiveLinkHubApplicationMode("CreatorMode", LOCTEXT("CreatorModeLabel", "Live Data"), App)
	{
		TabLayout = FTabManager::NewLayout("LiveLinkHubCreatorMode_v1.5");
		const TSharedRef<FTabManager::FArea> MainWindowArea = FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal);

		PanelController = MakeShared<FLiveLinkPanelController>(TAttribute<bool>::CreateRaw(this, &FLiveLinkHubCreatorAppMode::IsSourcePanelReadOnly));
		PanelController->OnSubjectSelectionChanged().AddRaw(this, &FLiveLinkHubCreatorAppMode::OnSubjectSelectionChanged);

		TabLayout->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(1.f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(UE::LiveLinkHub::SubjectsTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(UE::LiveLinkHub::SourceDetailsTabId, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(UE::LiveLinkHub::SubjectsTabId, ETabState::OpenedTab)
						->AddTab(UE::LiveLinkHub::RecordingListTabId, ETabState::OpenedTab)
						->SetForegroundTab(UE::LiveLinkHub::SubjectsTabId)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(UE::LiveLinkHub::ClientsTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(UE::LiveLinkHub::ClientDetailsTabId, ETabState::OpenedTab)
					)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->AddTab(UE::LiveLinkHub::PlaybackTabId, ETabState::ClosedTab)
			)
		);
		
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubDetailsTabSummoner>(App, PanelController));
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubDataDevicesTabSummoner>(App, PanelController));
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubPlaybackTabSummoner>(App));
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubRecordingListTabSummoner>(App));
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubClientsTabSummoner>(App));
		TabFactories.RegisterFactory(MakeShared<FLiveLinkHubClientDetailsTabSummoner>(App));
	}

	virtual ~FLiveLinkHubCreatorAppMode()
	{
		if (PanelController)
		{
			PanelController->OnSubjectSelectionChanged().RemoveAll(this);
		}

		PanelController.Reset();
	}

	//~ Begin FLiveLinkHubApplicationMode interface
	virtual FSlateIcon GetModeIcon() const
	{
		return FSlateIcon("LiveLinkStyle", TEXT("LiveLinkHub.Subjects.Icon"));
	}

	virtual TArray<TSharedRef<SWidget>> GetStatusBarWidgets_Impl() override
	{
		return {
			SNew(STextBlock)
			.Margin(FMargin(0.0, 0.0, 4.0, 0.0))
			.Text_Raw(this, &FLiveLinkHubCreatorAppMode::GetLoadedConfigText)
		};
	}
	//~ End FLiveLinkHubApplicationMode interface

private:
	/** Handles updating the active subject in the subject controller. */
	void OnSubjectSelectionChanged(const FLiveLinkSubjectKey& SubjectKey)
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		LiveLinkHubModule.GetSubjectController()->SetSubject(SubjectKey);
	}

	/** Returns whether the source panel is read only (such as when we're playing back a recording) */
	bool IsSourcePanelReadOnly() const
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		return (LiveLinkHubModule.GetPlaybackController()->IsInPlayback() && !GetDefault<ULiveLinkHubSettings>()->bAllowModifyingSourceSettingsInPlayback) || LiveLinkHubModule.GetRecordingController()->IsRecording();
	}

	/** Returns the name of the active session. */
	FText GetLoadedConfigText() const
	{
		if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkHub()->GetSessionManager())
		{
			const FString FileName = FPaths::GetBaseFilename(SessionManager->GetLastConfigPath());
			return FileName.IsEmpty() ? LOCTEXT("UntitledConfig", "Untitled") : FText::FromString(FileName);
		}

		return FText::GetEmpty();
	}

protected:

	/** Holds the livelink panel controller responsible for creating sources and subjects tabs. */
	TSharedPtr<FLiveLinkPanelController> PanelController;
};

#undef LOCTEXT_NAMESPACE
