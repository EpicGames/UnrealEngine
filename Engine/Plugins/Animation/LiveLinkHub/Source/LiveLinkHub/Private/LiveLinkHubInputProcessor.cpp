// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubInputProcessor.h"
#include "Input/Events.h"
#include "Misc/App.h"
#include "ViewportWorldInteraction.h"

namespace UE::LiveLinkHub::Private
{
	bool IsControllerButton( FName Name )
	{
		static FGamepadKeyNames::Type GamepadTypes[] = {
			FGamepadKeyNames::LeftAnalogX,
			FGamepadKeyNames::LeftAnalogY,
			FGamepadKeyNames::RightAnalogX,
			FGamepadKeyNames::RightAnalogY,
			FGamepadKeyNames::LeftTriggerAnalog,
			FGamepadKeyNames::RightTriggerAnalog,
			FGamepadKeyNames::LeftThumb,
			FGamepadKeyNames::RightThumb,
			FGamepadKeyNames::SpecialLeft,
			FGamepadKeyNames::SpecialLeft_X,
			FGamepadKeyNames::SpecialLeft_Y,
			FGamepadKeyNames::SpecialRight,
			FGamepadKeyNames::FaceButtonBottom,
			FGamepadKeyNames::FaceButtonRight,
			FGamepadKeyNames::FaceButtonLeft,
			FGamepadKeyNames::FaceButtonTop,
			FGamepadKeyNames::LeftShoulder,
			FGamepadKeyNames::RightShoulder,
			FGamepadKeyNames::LeftTriggerThreshold,
			FGamepadKeyNames::RightTriggerThreshold,
			FGamepadKeyNames::DPadUp,
			FGamepadKeyNames::DPadDown,
			FGamepadKeyNames::DPadRight,
			FGamepadKeyNames::DPadLeft,
			FGamepadKeyNames::LeftStickUp,
			FGamepadKeyNames::LeftStickDown,
			FGamepadKeyNames::LeftStickRight,
			FGamepadKeyNames::LeftStickLeft,
			FGamepadKeyNames::RightStickUp,
			FGamepadKeyNames::RightStickDown,
			FGamepadKeyNames::RightStickRight,
			FGamepadKeyNames::RightStickLeft
		};

		int32 Index = 0;
		const int32 NumberOfTypes = sizeof(GamepadTypes)/sizeof(FGamepadKeyNames::Type);
		bool bIsGamepadType = false;
		while (Index < NumberOfTypes && !bIsGamepadType)
		{
			bIsGamepadType = GamepadTypes[Index] == Name;
			Index++;
		}

		return bIsGamepadType;
	}
}

bool FLiveLinkHubInputProcessor::HandleKeyDownEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent )
{
	return UE::LiveLinkHub::Private::IsControllerButton(InKeyEvent.GetKey().GetFName());
}

bool FLiveLinkHubInputProcessor::HandleKeyUpEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent )
{
	return UE::LiveLinkHub::Private::IsControllerButton(InKeyEvent.GetKey().GetFName());
}

bool FLiveLinkHubInputProcessor::HandleAnalogInputEvent( FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent )
{
	return true;
}

