// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GenericPlatform/ICursor.h"
#include "Framework/Application/IInputProcessor.h"

class FSlateApplication;
class FViewportWorldInteractionManager;
struct FAnalogInputEvent;
struct FKeyEvent;
struct FPointerEvent;

/** This class will allow us to suppress gamepad input events for the slate application.*/
class FLiveLinkHubInputProcessor : public IInputProcessor
{
public:
	FLiveLinkHubInputProcessor() = default;
	virtual ~FLiveLinkHubInputProcessor() = default;

	//~ IInputProcess overrides
	virtual bool HandleKeyDownEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent ) override;
	virtual bool HandleKeyUpEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent ) override;
	virtual bool HandleAnalogInputEvent( FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent ) override;

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}

	virtual const TCHAR* GetDebugName() const override { return TEXT("LiveLinkHubInputProcessor"); }
};
