// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkHub, Log, All);

DECLARE_STATS_GROUP(TEXT("Live Link Hub"), STATGROUP_LiveLinkHub, STATCAT_Advanced);