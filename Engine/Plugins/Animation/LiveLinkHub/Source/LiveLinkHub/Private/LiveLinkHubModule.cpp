// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubModule.h"

#include "CoreGlobals.h"
#include "Clients/LiveLinkHubProvider.h"
#include "LevelEditor.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkHubSubjectSettings.h"
#include "LiveLinkSettings.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "Recording/LiveLinkHubRecordingController.h"
#include "Recording/LiveLinkHubRecordingMountManager.h"
#include "Settings/LiveLinkHubSettings.h"
#include "Settings/LiveLinkHubClientTextFilterCustomization.h"
#include "Settings/LiveLinkHubSettingsCustomization.h"
#include "Settings/LiveLinkSettingsCustomization.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubModule"

void FLiveLinkHubModule::StartupModule()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StartLiveLinkHub);

	const bool bConfigValue = bCreateLiveLinkHubInstance = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);

	bCreateLiveLinkHubInstance = bCreateLiveLinkHubInstance && !IsRunningCommandlet() && FApp::CanEverRender();
	UE_LOGF(LogLiveLinkHub, Display, "LiveLinkHubModule::StartupModule - LiveLinkHub instance %ls (Config: %d)", bCreateLiveLinkHubInstance ? TEXT("will be created.") : TEXT("will not be created."), (int32) bConfigValue);

	if (bCreateLiveLinkHubInstance)
	{
		// Needed for downstream modules that need the editor commands to be loaded.
		FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		if (!GetDefault<ULiveLinkHubSettings>()->bTickOnGameThread)
		{
			Ticker.StartTick();
		}

		LiveLinkHub = MakeShared<FLiveLinkHub>();
		LiveLinkHub->Initialize(Ticker);
		LiveLinkHub->OnFrameReceived().BindRaw(&Ticker, &FLiveLinkHubTicker::TriggerUpdate);

		RecordingMountManager = MakeUnique<FLiveLinkHubRecordingMountManager>();
	}

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(ULiveLinkHubSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkHubSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FLiveLinkHubClientTextFilter::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkHubClientTextFilterCustomization::MakeInstance));
	// Apply our customization for core live link settings, only if we aren't running in the full editor. We hide properties that aren't
	// supported in a standalone application context, but are needed if loaded in the editor.
	bUseSettingsDetailCustomization =
		GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bUseLiveLinkHubSettingsDetailCustomization"), false, GEngineIni);
	if (bUseSettingsDetailCustomization)
	{
		PropertyModule.RegisterCustomClassLayout(ULiveLinkSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkSettingsCustomization::MakeInstance));
	}	
}

void FLiveLinkHubModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FLiveLinkHubClientTextFilter::StaticStruct()->GetFName());
			PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkHubSettings::StaticClass()->GetFName());
			if (bUseSettingsDetailCustomization)
			{
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkSettings::StaticClass()->GetFName());
			}
		}
	}

	if (bCreateLiveLinkHubInstance)
	{
		LiveLinkHub->OnFrameReceived().Unbind();

		Ticker.Stop();
		Ticker.Exit();

		RecordingMountManager.Reset();
		LiveLinkHub.Reset();
	}
}

TSharedPtr<FLiveLinkHub> FLiveLinkHubModule::GetLiveLinkHub() const
{
	return LiveLinkHub;
}

TSharedPtr<FLiveLinkHubProvider> FLiveLinkHubModule::GetLiveLinkProvider() const
{
	return LiveLinkHub ? LiveLinkHub->LiveLinkProvider : nullptr;
}

TSharedPtr<FLiveLinkHubRecordingController> FLiveLinkHubModule::GetRecordingController() const
{
	return LiveLinkHub ? LiveLinkHub->RecordingController : nullptr;
}

TSharedPtr<FLiveLinkHubRecordingListController> FLiveLinkHubModule::GetRecordingListController() const
{
	return LiveLinkHub ? LiveLinkHub->RecordingListController : nullptr;
}

TSharedPtr<FLiveLinkHubPlaybackController> FLiveLinkHubModule::GetPlaybackController() const
{
	return LiveLinkHub ? LiveLinkHub->PlaybackController : nullptr;
}

TSharedPtr<FLiveLinkHubSubjectController> FLiveLinkHubModule::GetSubjectController() const
{
	return LiveLinkHub ? LiveLinkHub->SubjectController : nullptr;
}

TSharedPtr<ILiveLinkHubSessionManager> FLiveLinkHubModule::GetSessionManager() const
{
	return LiveLinkHub ? LiveLinkHub->SessionManager : nullptr;
}

FLiveLinkHubRecordingMountManager* FLiveLinkHubModule::GetRecordingMountManager() const
{
	return RecordingMountManager.Get();
}

IMPLEMENT_MODULE(FLiveLinkHubModule, LiveLinkHub);

#undef LOCTEXT_NAMESPACE /* LiveLinkHubModule */
