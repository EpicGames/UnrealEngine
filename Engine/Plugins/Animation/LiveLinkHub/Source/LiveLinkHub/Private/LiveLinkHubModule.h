// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkHubModule.h"
#include "LiveLinkHubTicker.h"
#include "Templates/SharedPointer.h"

class FLiveLinkHub;
class FLiveLinkHubPlaybackController;
class FLiveLinkHubProvider;
class FLiveLinkHubRecordingController;
class FLiveLinkHubRecordingListController;
class FLiveLinkHubRecordingMountManager;
class FLiveLinkHubSubjectController;
class FUICommandList;
class ILiveLinkHubSessionManager;

class FLiveLinkHubModule : public ILiveLinkHubModule
{
public:
	//~ Begin ILiveLinkHubModule interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End ILiveLinkHubModule interface

	/** Get the livelink hub object. */
	TSharedPtr<FLiveLinkHub> GetLiveLinkHub() const;
	/** Get the livelink provider responsible for forwarding livelink data to connected UE clients. */
	TSharedPtr<FLiveLinkHubProvider> GetLiveLinkProvider() const;
	/** Get the recording controller. */
	TSharedPtr<FLiveLinkHubRecordingController> GetRecordingController() const;
	/** Get the recording list controller. */
	TSharedPtr<FLiveLinkHubRecordingListController> GetRecordingListController() const;
	/** Get the playback controller. */
	TSharedPtr<FLiveLinkHubPlaybackController> GetPlaybackController() const;
	/** Get the subject controller. */
	TSharedPtr<FLiveLinkHubSubjectController> GetSubjectController() const;
	/** Get the subject controller. */
    TSharedPtr<ILiveLinkHubSessionManager> GetSessionManager() const;
	/** Get the recording mount manager. */
	FLiveLinkHubRecordingMountManager* GetRecordingMountManager() const;

private:
	/** LiveLinkHub object responsible for initializing the different controllers. */
	TSharedPtr<FLiveLinkHub> LiveLinkHub;

	/** Utility object used to tick LiveLink outside of the game thread. */
	FLiveLinkHubTicker Ticker;

	/** Manages custom mount points for recording directories. */
	TUniquePtr<FLiveLinkHubRecordingMountManager> RecordingMountManager;

	/** Config-driven. True in LiveLink Hub standalone application, false in UE. */
	bool bUseSubjectSettingsDetailsCustomization = false;
	
	/** Overrides standard livelink settings customization. Config-driven. True in LiveLink Hub standalone application, false in UE. */
	bool bUseSettingsDetailCustomization = false;

	/** Whether or not to create a LiveLinkHub instance (false in editor). */
	bool bCreateLiveLinkHubInstance = false;
};
