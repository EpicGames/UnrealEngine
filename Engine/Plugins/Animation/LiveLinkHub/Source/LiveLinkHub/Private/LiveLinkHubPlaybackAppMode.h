// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubApplicationMode.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Docking/TabManager.h"
#include "LiveLinkHubApplicationBase.h"
#include "LiveLinkHub.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubPlaybackMode"

/** Compact mode focused exclusively on playing back recordings. */
class FLiveLinkHubPlaybackAppMode : public FLiveLinkHubApplicationMode
{
public:
	FLiveLinkHubPlaybackAppMode(TSharedPtr<FLiveLinkHubApplicationBase> App)
		: FLiveLinkHubApplicationMode("PlaybackMode", LOCTEXT("PlaybackModeLabel", "Playback"), App)
	{
		TabLayout = FTabManager::NewLayout("LiveLinkHubPlaybackMode_v1.0");
		TabLayout->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(1.f)
				->AddTab(UE::LiveLinkHub::PlaybackTabId, ETabState::OpenedTab)
			)
		);
	}

	//~ Begin FLiveLinkHubApplicationMode interface
	virtual void PostActivateMode() override
	{
		if (TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController = FLiveLinkHub::Get()->GetPlaybackController())
		{
			PlaybackController->SetClosePlaybackTabOnEject(false);
			
			TSharedRef<SWindow> RootWindow = FLiveLinkHub::Get()->GetRootWindow();

			PreviousWindowSize = RootWindow->GetSizeInScreen();
			FVector2D WindowPosition = RootWindow->GetPositionInScreen();
			FVector2D TargetSize = FVector2D{ 800.0, 220.0 } * FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowPosition.X, WindowPosition.Y);

			RootWindow->Resize(TargetSize);
		}
	}

	virtual void PreDeactivateMode() override
	{
		if (TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController = FLiveLinkHub::Get()->GetPlaybackController())
		{
			PlaybackController->SetClosePlaybackTabOnEject(true);

			if (PreviousWindowSize.Size() != 0)
			{
				FLiveLinkHub::Get()->GetRootWindow()->Resize({ PreviousWindowSize.X, PreviousWindowSize.Y });
			}
		}
	}

	virtual FSlateIcon GetModeIcon() const
	{
		return FSlateIcon(UE::LiveLinkHub::LiveLinkStyleName, TEXT("LiveLinkHub.Playback.Icon"));
	}
	//~ End FLiveLinkHubApplicationMode interface

private:
	FVector2d PreviousWindowSize;
};

#undef LOCTEXT_NAMESPACE
