// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Clients/LiveLinkHubProvider.h"
#include "Features/IModularFeatures.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubClient.h"
#include "LiveLinkTypes.h"

class FLiveLinkHubSubjectSettingsUtils
{
public:
	/** Notify connected clients that this subject's name has changed. */
	static bool ValidateOutboundName(const FString& SubjectName, FName PreviousOutboundName, const FString& InOutboundNameCandidate)
	{
		if (InOutboundNameCandidate.IsEmpty() || FName(InOutboundNameCandidate) == NAME_None)
		{
			return false;
		}

		if (InOutboundNameCandidate == SubjectName)
		{
			return true;
		}

		FLiveLinkHubClient* LiveLinkClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));

		// Can't rename to an existing subject, so check if it exists by querying for its role.
		return LiveLinkClient->GetSubjectRole_AnyThread(FLiveLinkSubjectName(*InOutboundNameCandidate)).Get() == nullptr;
	}

	/** Returns whether a new name candidate for the outbound name is valid. */
	static void NotifyRename(FName PreviousOutboundName, const FString& OutboundName, const FLiveLinkSubjectKey& SubjectKey)
	{
		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

		if (TSharedPtr<FLiveLinkHubProvider> Provider = LiveLinkHubModule.GetLiveLinkProvider())
		{
			// Re-send the last static data with the new name.
			TPair<UClass*, FLiveLinkStaticDataStruct*> StaticData = Provider->GetLastSubjectStaticDataStruct(PreviousOutboundName);
			if (StaticData.Key && StaticData.Value)
			{
				FLiveLinkStaticDataStruct StaticDataCopy;
				StaticDataCopy.InitializeWith(*StaticData.Value);


				TMap<FName, FString> ExtraAnnotations;

				FLiveLinkHubClient* LiveLinkClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
				const FText OriginalSourceType = LiveLinkClient->GetSourceType(SubjectKey.Source);
				ExtraAnnotations.Add(FLiveLinkMessageAnnotation::OriginalSourceAnnotation, OriginalSourceType.ToString());
				Provider->UpdateSubjectStaticData(*OutboundName, StaticData.Key, MoveTemp(StaticDataCopy), ExtraAnnotations);
			}

			// Then clear the old static data entry in the provider.
			Provider->RemoveSubject(PreviousOutboundName);
		}
	}

};