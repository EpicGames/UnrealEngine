// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "LiveLinkHub.h"
#include "Misc/Timespan.h"
#include "Settings/LiveLinkHubSettings.h"

/** Object used to tick LiveLinkHub outside of the game thread. */
class FLiveLinkHubTicker : public FRunnable
{
public:
	void StartTick()
	{
		if (!bIsRunning)
		{
			bIsRunning = true;
			TickEvent = FGenericPlatformProcess::GetSynchEventFromPool();
			check(TickEvent);
			Thread.Reset(FRunnableThread::Create(this, TEXT("LiveLinkHubTicker")));
		}
	}

	//~ Begin FRunnable Interface
	virtual uint32 Run() override
	{
		float TickFrequency = 1 / GetDefault<ULiveLinkHubSettings>()->TargetFrameRate;
		FTimespan TickTimeSpan = FTimespan::FromSeconds(TickFrequency);

		while (bIsRunning)
		{
			check(TickEvent);

			bool bWasTriggeredTick = TickEvent->Wait(TickTimeSpan);
			TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(FLiveLinkHubTicker::TriggeredTick, bWasTriggeredTick);
			TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(FLiveLinkHubTicker::ScheduledTick, !bWasTriggeredTick);

			if (bIsRunning) // make sure we were not told to exit during the wait
			{
				OnTickDelegate.Broadcast();
			}
		}

		return 0;
	}
	virtual void Exit() override
	{
		if (!bIsRunning)
		{
			return;
		}

		bIsRunning = false;

		if (TickEvent)
		{
			TickEvent->Trigger();
		}

		Thread->WaitForCompletion();

		if (TickEvent)
		{
			FGenericPlatformProcess::ReturnSynchEventToPool(TickEvent);
			TickEvent = nullptr;
		}
	}
	//~ End FRunnable Interface

	void TriggerUpdate()
	{
		if (TickEvent)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHubTicker::TriggerUpdate);
			TickEvent->Trigger();
		}
	}

public:

	/** Get the delegate called whenever this object ticks. */
	FTSSimpleMulticastDelegate& OnTick() { return OnTickDelegate;  }
private:

	std::atomic<bool> bIsRunning = false;

	/** Delegate called when this ticks. */
	FTSSimpleMulticastDelegate OnTickDelegate;

	FEvent* TickEvent = nullptr;
	TUniquePtr<FRunnableThread> Thread;
};
