// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubUserLayoutMode.h"

#include "Dom/JsonObject.h"
#include "LiveLinkHub.h"

FLiveLinkHubUserLayoutMode::FLiveLinkHubUserLayoutMode(FName InLayoutName, TSharedRef<FJsonObject> UserLayout, TSharedPtr<FLiveLinkHubApplicationMode> InParentMode)
	: FLiveLinkHubApplicationMode(InLayoutName, FText::FromName(InLayoutName), FLiveLinkHub::Get())
	, ParentMode(MoveTemp(InParentMode))
{
	LayoutExtender = ParentMode->LayoutExtender;
	ToolbarExtender = ParentMode->GetToolbarExtender();

	ParentLayoutName = UserLayout->GetStringField(TEXT("Name"));

	// Note: At runtime, we *must* override the layout name so that it doesn't clash with the parent mode's layout name.
	// Otherwise, we would run into issues where modifying this layout would also affect the parent or vice-versa.
	UserLayout->SetStringField(TEXT("Name"), GetModeName().ToString());

	TabLayout = FTabManager::FLayout::NewFromJson(UserLayout);
}

void FLiveLinkHubUserLayoutMode::PreDeactivateMode()
{
	FLiveLinkHubApplicationMode::PreDeactivateMode();

	TSharedPtr<FTabManager::FLayout> PersistentLayout = FLiveLinkHub::Get()->GetTabManager()->PersistLayout();
	TSharedPtr<FJsonObject> JsonLayout = PersistentLayout->ToJson();

	// Save out the layout with the original name.
	JsonLayout->SetStringField(TEXT("Name"), ParentLayoutName);
	FLiveLinkHub::Get()->PersistUserLayout(ModeName.ToString(), JsonLayout);
}
