// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubApplicationMode.h"

#include "Dom/JsonObject.h"

/** ApplicationMode that represents a user layout. It acts as a proxy for its parent mode. */
class FLiveLinkHubUserLayoutMode : public FLiveLinkHubApplicationMode
{
public:
	FLiveLinkHubUserLayoutMode(FName InLayoutName, TSharedRef<FJsonObject> UserLayout, TSharedPtr<FLiveLinkHubApplicationMode> InParentMode);

	/** Get the parent mode that holds the actual functionality (ie. Creator mode, Capture Manager) */
	const TSharedPtr<FLiveLinkHubApplicationMode>& GetParentMode() const
	{
		return ParentMode;
	}

	//~ Begin FLiveLinkHubApplicationMode interface
	virtual FSlateIcon GetModeIcon() const override 
	{ 
		return ParentMode->GetModeIcon();
	}

	virtual bool IsUserLayout() const override
	{
		return true;
	}

	virtual TArray<TSharedRef<SWidget>> GetStatusBarWidgets_Impl() override
	{
		return ParentMode->GetStatusBarWidgets_Impl();
	}

	virtual TArray<TSharedRef<SWidget>> GetToolbarWidgets_Impl() override
	{
		return ParentMode->GetToolbarWidgets_Impl();
	}

	virtual void PreDeactivateMode() override;
	//~ End FLiveLinkHubApplicationMode interface

private:
	/** The mode that holds the functionality and tab spawners for this mode. */
	TSharedPtr<FLiveLinkHubApplicationMode> ParentMode;
	/** Layout name of the parent mode (ie. LiveLinkHubCreatorMode_V1.0) */
	FString ParentLayoutName;
};