// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFrameData.h"

#include "Settings/LiveLinkHubSettings.h"

namespace UE::LiveLinkHub::FrameData::Private
{
	void FFrameBufferCache::TrimCache()
	{
		const int32 MaxHistory = GetDefault<ULiveLinkHubSettings>()->PlaybackMaxBufferRangeHistory;
		const int32 AmountToTrim = FrameData.Num() - MaxHistory;
		if (AmountToTrim > 0)
		{
			FrameData.RemoveAt(0, AmountToTrim);
		}
	}

	void FFrameBufferCache::CleanCache(const TRange<int32>& InActiveRange)
	{
		for (FLiveLinkRecordingBaseDataContainer& Container : FrameData)
		{
			const TRange<int32> BufferedRange = Container.GetBufferedFrames();

			// Clear portions that are completely outside the active range.
			if (BufferedRange.GetUpperBoundValue() < InActiveRange.GetLowerBoundValue()
				|| BufferedRange.GetLowerBoundValue() > InActiveRange.GetUpperBoundValue())
			{
				Container.ClearData();
			}
			else
			{
				// In this case we remove frames beyond the intersection with the active range,
				// while keeping frames that are still relevant.
				TRange<int32> Intersection = TRange<int32>::Intersection(InActiveRange, BufferedRange);
				Container.RemoveFramesAfter(Intersection.GetUpperBoundValue());
				Container.RemoveFramesBefore(Intersection.GetLowerBoundValue());
			}
		}
		
		// Cache data is also cleared at a per frame level when checking history.
		FrameData.RemoveAll([](const FLiveLinkRecordingBaseDataContainer& Container)
		{
			return Container.IsEmpty();
		});

		TrimCache();
	}

	RangeHelpers::Private::TRangeArray<int32> FFrameBufferCache::GetCacheBufferRanges(bool bIncludeOffset) const
	{
		RangeHelpers::Private::TRangeArray<int32> Results;
		for (const FLiveLinkRecordingBaseDataContainer& Container : FrameData)
		{
			Results.Add(Container.GetBufferedFrames(bIncludeOffset));
		}

		return Results;
	}

	TSharedPtr<FInstancedStruct> FFrameBufferCache::TryGetCachedFrame(const int32 InFrame, double& OutTimestamp)
	{
		for (FLiveLinkRecordingBaseDataContainer& Data : FrameData)
		{
			if (TSharedPtr<FInstancedStruct> Result = Data.TryGetFrame(InFrame, OutTimestamp))
			{
				return Result;
			}
		}

		return nullptr;
	}

	bool FFrameBufferCache::ContainsFrame(const int32 InFrame) const
	{
		for (const FLiveLinkRecordingBaseDataContainer& Data : FrameData)
		{
			if (Data.IsFrameLoaded(InFrame))
			{
				return true;
			}
		}

		return false;
	}
}
