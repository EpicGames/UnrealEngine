// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkRecordingDataContainer.h"
#include "Recording/LiveLinkRecordingRangeHelpers.h"
#include "StructUtils/InstancedStruct.h"

namespace UE::LiveLinkHub::FrameData::Private
{
	/** Data manager for frames which have been loaded. */
	struct FFrameBufferCache
	{
		/** Loaded frame data. */
		TArray<FLiveLinkRecordingBaseDataContainer> FrameData;

		/** Ensure the size of the cache is limited, removing older entries. */
		void TrimCache();

		/** Remove data from the cache that is no longer needed. */
		void CleanCache(const TRange<int32>& InActiveRange);

		/**
		 * Return all ranges contained within the cache.
		 * @param bIncludeOffset If the local frame time-to-frame index offset used should be added to the ranges.
		 */
		RangeHelpers::Private::TRangeArray<int32> GetCacheBufferRanges(bool bIncludeOffset = false) const;

		/** Look through the cache for a loaded frame. */
		TSharedPtr<FInstancedStruct> TryGetCachedFrame(const int32 InFrame, double& OutTimestamp);

		/** Checks if a frame is contained within the cache. */
		bool ContainsFrame(const int32 InFrame) const;
	};

	/** Keeps track of current frame iterations while buffering. */
	struct FFrameBufferIterationData
	{
		enum ECompletionStatus
		{
			// New iteration, no processing has performed yet.
			New,
			// Iteration is currently active.
			Active,
			// Iteration has been canceled for any reason.
			Canceled,
			// Iteration has fully completed, buffering all available requested frames.
			Complete
		};

		/** Temporary storage for loaded frames, which gets moved to data containers after loading. */
		struct FTemporaryData
		{
			/** Fully processed frame data. */
			TArray<TSharedPtr<FInstancedStruct>> RecordedData;
			/** Timestamps corresponding to the frame data. */
			TArray<double> Timestamps;
			void Reset()
			{
				RecordedData.Empty();
				Timestamps.Empty();
			}
		};

		/** The last loaded left frame. */
		int32 LastLoadedLeftFrame;
		/** The last loaded right frame. */
		int32 LastLoadedRightFrame;
		/** If we are loading to the right. */
		bool bLoadRight;
		/** Current status of this iteration. */
		ECompletionStatus Status;

		/** Forward-looking frame data. */
		FTemporaryData ForwardData;
		/** Reverse looking frame data. */
		FTemporaryData ReverseData;

		FFrameBufferIterationData()
		{
			Reset();
		}

		/** Resets the iteration to a new iteration, clearing all storage. */
		void Reset()
		{
			Status = New;
			LastLoadedLeftFrame = INDEX_NONE;
			LastLoadedRightFrame = INDEX_NONE;
			bLoadRight = true;
			ForwardData.Reset();
			ReverseData.Reset();
		}
	};
	
	/** Frame data information when loading from a recording file. */
	struct FFrameMetaData
	{
		/** The subject key used for the frame data. */
		TSharedPtr<FLiveLinkSubjectKey> FrameDataSubjectKey;
		/** The struct for this frame data. */
		TWeakObjectPtr<UScriptStruct> LoadedStruct;
		/** The position in the file recording where frame data begins. */
		int64 RecordingStartFrameFilePosition = 0;
		/** Maximum number of frames. */
		int32 MaxFrames = 0;
		/** Frame offsets and sizes. [FrameOffsetBytes, FrameSizeBytes] */
		TArray<TTuple<int64, int32>> FrameDiskSizes;
		/** All timestamps for the track, whether the frame is loaded or not. */
		TArray<double> Timestamps;
		/** Whether the frame size is consistent throughout this animation. */
		bool bHasConsistentFrameSize = false;
		/** The last timestamp for this frame data. */
		double LastTimestamp = 0.f;
		/** The frame rate, based only off of number of frames and the last timestamp. */
		FFrameRate LocalFrameRate;
		/** Cache of previously buffered frames, which are not currently active. */
		FFrameBufferCache BufferedCache;
		/** Current iteration data while buffering. */
		FFrameBufferIterationData BufferIterationData;
		/** Name of the compression method used on the frame data. */
		FName CompressionMethodName;

		/**
		 * Find the closest frame matching the given timestamp.
		 * @param InTimestamp The timestamp of the frame.
		 * @param InIndexHint A hint to where the index might be, allowing quicker search times.
		 */
		int32 FindFrameFromTimestamp(const double InTimestamp, const int32 InIndexHint) const
		{
			const int32 Num = Timestamps.Num();
			if (Num == 0)
			{
				ensure(false);
				return INDEX_NONE;
			}

			// Check ranges.
			if (InTimestamp <= Timestamps[0])
			{
				return 0;
			}
			if (InTimestamp >= Timestamps[Num - 1])
			{
				return Num - 1;
			}

			const int32 Hint = FMath::Clamp(InIndexHint, 0, Num - 1);
			
			// Are we already hitting the target?
			const double HintTime = Timestamps[Hint];
			const bool bIsLast = (Hint == Num - 1);
			if (HintTime <= InTimestamp &&
				(bIsLast || Timestamps[Hint + 1] > InTimestamp))
			{
				return Hint;
			}

			int32 Low, High;
			int32 Best = 0;
			if (HintTime < InTimestamp)
			{
				// Search to the right.
				Low  = Hint + 1;
				High = Num - 1;
				Best = Hint;
			}
			else
			{
				// Search to the left.
				Low  = 0;
				High = Hint - 1;
				Best = 0;
			}

			while (Low <= High)
			{
				const int32 Mid = (Low + High) / 2;
				const double MidTime = Timestamps[Mid];

				if (MidTime == InTimestamp)
				{
					return Mid;
				}
				if (MidTime < InTimestamp)
				{
					Best = Mid;
					Low  = Mid + 1; // Go right.
				}
				else
				{
					High = Mid - 1; // Go left.
				}
			}

			return Best;
		}
		
		/** The size in bytes of an animation frame. */
		int32 GetFrameDiskSize(const int32 InFrameIdx) const
		{
			check(InFrameIdx >= 0 && InFrameIdx < FrameDiskSizes.Num());
			return FrameDiskSizes[InFrameIdx].Value;
		}
		
		/** Find the correct file offset based on the frame index. */
		int64 GetFrameFilePosition(const int32 InFrameIdx) const
		{
			return RecordingStartFrameFilePosition + GetRelativeFrameFilePosition(InFrameIdx);
		};

		/** Find the offset relative to local storage only, not accounting for disk position. */
		int64 GetRelativeFrameFilePosition(const int32 InFrameIdx) const
		{
			check(InFrameIdx >= 0 && InFrameIdx < FrameDiskSizes.Num());
			return bHasConsistentFrameSize ? static_cast<int64>(FrameDiskSizes[InFrameIdx].Value) * static_cast<int64>(InFrameIdx) : FrameDiskSizes[InFrameIdx].Key;
		};
	};
}