// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkRecordingDataContainer.h"

#include "Recording/LiveLinkRecordingRangeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkRecordingDataContainer)

bool FLiveLinkRecordingBaseDataContainer::IsEmpty() const
{
	check(Timestamps.Num() == RecordedData.Num());
	return Timestamps.Num() == 0;
}

void FLiveLinkRecordingBaseDataContainer::ClearData()
{
	Timestamps.Empty();
	RecordedData.Empty();
	RecordedDataStartFrame = 0;
}

TRange<int32> FLiveLinkRecordingBaseDataContainer::GetBufferedFrames(bool bIncludeOffset) const
{
	if (Timestamps.Num() == 0)
	{
		return TRange<int32>::Empty();
	}
	int32 StartFrame = RecordedDataStartFrame;
	int32 EndFrame = FMath::Max(0, RecordedDataStartFrame + Timestamps.Num() - 1);
	if (bIncludeOffset && LocalFrameOffset > 0)
	{
		// We only add on positive frame offsets because negative ones, while accurate, don't correctly report this track's frames
		// at a global level. So if we're waiting for high frames to buffer that don't really exist because they actually occurred in the past, we
		// will freeze indefinitely. Such as we need frame 5000, but only one track has this frame at a negative offset because it ran at a
		// faster framerate, so it's already happened and converts to a lower frame.
		// @todo: We should revamp how loaded frames are reported, or how we wait for a frame range to buffer.
		
		StartFrame += LocalFrameOffset;
		EndFrame += LocalFrameOffset;
	}
	return UE::LiveLinkHub::RangeHelpers::Private::MakeInclusiveRange(StartFrame, EndFrame);
}

TSharedPtr<FInstancedStruct> FLiveLinkRecordingBaseDataContainer::TryGetFrame(const int32 InFrame) const
{
	if (IsFrameLoaded(InFrame))
	{
		const int32 RelativeFrameIdx = GetRelativeFrameIndex(InFrame);
		return RecordedData[RelativeFrameIdx];
	}
	return nullptr;
}

TSharedPtr<FInstancedStruct> FLiveLinkRecordingBaseDataContainer::TryGetFrame(const int32 InFrame, double& OutTimestamp) const
{
	if (TSharedPtr<FInstancedStruct> OutFrame = TryGetFrame(InFrame))
	{
		const int32 RelativeFrameIdx = GetRelativeFrameIndex(InFrame);
		OutTimestamp = Timestamps[RelativeFrameIdx];
		return OutFrame;
	}
	return nullptr;
}

void FLiveLinkRecordingBaseDataContainer::RemoveFramesBefore(const int32 InEndFrame)
{
	const int32 RelativeFrameIdx = FMath::Min(GetRelativeFrameIndex(InEndFrame), RecordedData.Num() - 1);
	if (!IsEmpty() && RelativeFrameIdx >= 0)
	{
		const int32 AmountToRemove = RelativeFrameIdx + 1;
		
		RecordedData.RemoveAt(0, AmountToRemove);
		Timestamps.RemoveAt(0, AmountToRemove);

		RecordedDataStartFrame += AmountToRemove;
	}
}

void FLiveLinkRecordingBaseDataContainer::RemoveFramesAfter(const int32 InStartFrame)
{
	const int32 RelativeFrameIdx = GetRelativeFrameIndex(InStartFrame);
	if (!IsEmpty() && RelativeFrameIdx >= 0 && RelativeFrameIdx < RecordedData.Num())
	{
		const int32 AmountToRemove = RecordedData.Num() - RelativeFrameIdx;
		
		RecordedData.RemoveAt(RelativeFrameIdx, AmountToRemove);
		Timestamps.RemoveAt(RelativeFrameIdx, AmountToRemove);
	}
}

void FLiveLinkRecordingBaseDataContainer::ValidateData() const
{
	check(Timestamps.Num() == RecordedData.Num());
	for (const TSharedPtr<FInstancedStruct>& InstancedStruct : RecordedData)
	{
		check(InstancedStruct.IsValid() && InstancedStruct->IsValid());
	}
}
