// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "StructUtils/InstancedStruct.h"

#include "LiveLinkRecordingDataContainer.generated.h"

/** Base data container for a recording track. */
USTRUCT()
struct FLiveLinkRecordingBaseDataContainer
{
	GENERATED_BODY()

	/** SERIALIZED DATA - Timestamps for the recorded data. Each entry matches an entry in the RecordedData array. */
	TArray<double> Timestamps;
	
	/**
	 * SERIALIZED DATA - Array of either static or frame recorded for a given timestamp.
	 * TSharedPtr used as streaming the data in may require shared access.
	 * FSharedStruct is not used because it doesn't implement Serialize().
	 */
	TArray<TSharedPtr<FInstancedStruct>> RecordedData;

	/** The current start frame for RecordedData. */
	int32 RecordedDataStartFrame = 0;

	/** Offset which was applied locally during frame rate conversions to find the correct timestamp. */
	int32 LocalFrameOffset = 0;

	/** If this container contains no data. */
	bool IsEmpty() const;

	/** Empty all data. */
	void ClearData();

	/**
	 * Retrieve the total buffered frames.
	 * @param bIncludeOffset If the local frame time-to-frame index offset used should be added to the ranges.
	 */
	TRange<int32> GetBufferedFrames(bool bIncludeOffset = false) const;

	/**
	 * Retrieve a loaded frame.
	 *
	 * @param InFrame The absolute frame index to load.
	 *
	 * @return The frame if it exists.
	 */
	TSharedPtr<FInstancedStruct> TryGetFrame(const int32 InFrame) const;

	/**
	 * Retrieve a loaded frame.
	 *
	 * @param InFrame The absolute frame index to load.
	 * @param OutTimestamp Return the timestamp, if it exists.
	 *
	 * @return The frame if it exists.
	 */
	TSharedPtr<FInstancedStruct> TryGetFrame(const int32 InFrame, double& OutTimestamp) const;

	/**
	 * Remove all frames before, and including, the input frame.
	 * @param InEndFrame The final frame to remove, inclusive.
	 */
	void RemoveFramesBefore(const int32 InEndFrame);

	/**
	 * Remove all frames including and after the input frame.
	 * @param InStartFrame The first frame to remove, inclusive.
	 */
	void RemoveFramesAfter(const int32 InStartFrame);

	/** Convert the absolute frame to the relative index for this data container. */
	int32 GetRelativeFrameIndex(const int32 InFrame) const
	{
		return InFrame - RecordedDataStartFrame;
	}
	
	/**
	 * Checks if a frame is currently loaded.
	 * 
	 * @param InFrame The absolute frame index to check.
	 *
	 * @return True if the frame index exists within the frame array.
	 */
	bool IsFrameLoaded(const int32 InFrame) const
	{
		return InFrame >= RecordedDataStartFrame && InFrame < RecordedDataStartFrame + RecordedData.Num();
	}

	/** Check data memory is valid and expected. */
	void ValidateData() const;
};

/** Container for static data. */
USTRUCT()
struct FLiveLinkRecordingStaticDataContainer : public FLiveLinkRecordingBaseDataContainer
{
	GENERATED_BODY()

	/** The role of the static data being recorded. */
	UPROPERTY()
	TSubclassOf<ULiveLinkRole> Role;
};

USTRUCT()
struct FLiveLinkUAssetRecordingData
{
	GENERATED_BODY()

	/** Length of the recording in seconds. */
	UPROPERTY()
	double LengthInSeconds = 0;

	/** Static data encountered while recording. */
	UPROPERTY()
	TMap<FLiveLinkSubjectKey, FLiveLinkRecordingStaticDataContainer> StaticData;
	
	/** Frame data encountered while recording. */
	UPROPERTY()
	TMap<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer> FrameData;
};
