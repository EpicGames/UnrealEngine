// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkUAssetRecorder.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Config/LiveLinkHubFileUtilities.h"
#include "Containers/Map.h"
#include "ContentBrowserModule.h"
#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "FileHelpers.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformFileManager.h"
#include "IContentBrowserSingleton.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubClient.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "LiveLinkPreset.h"
#include "LiveLinkTypes.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "NamingTokensEngineSubsystem.h"
#include "Recording/Implementations/LiveLinkUAssetRecording.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "Recording/LiveLinkRecording.h"
#include "Session/LiveLinkHubSessionData.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "Settings/LiveLinkHubSettings.h"
#include "StructUtils/InstancedStruct.h"
#include "UI/Window/LiveLinkHubWindowController.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.AssetRecorder"


namespace UE::AssetRecorderUtils::Private
{
	/** Get the current recording mount point (custom or default /Game) */
	FString GetRecordingMountPoint()
	{
		FString RecordingMountPoint = TEXT("/Game");
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = HubModule.GetSessionManager())
		{
			if (TSharedPtr<ILiveLinkHubSession> Session = SessionManager->GetCurrentSession())
			{
				if (ULiveLinkHubSessionData* SessionData = Session->GetSessionData())
				{
					if (SessionData->bUseCustomRecordingDirectory && !SessionData->CustomRecordingMountPoint.IsEmpty())
					{
						RecordingMountPoint = SessionData->CustomRecordingMountPoint;
					}
				}
			}
		}
		return RecordingMountPoint;
	}

	TOptional<FLiveLinkRecordingStaticDataContainer> CreateStaticDataContainerFromFrameData(const FLiveLinkSubjectKey& SubjectKey)
	{
		TOptional<FLiveLinkRecordingStaticDataContainer> StaticDataContainer;
		FLiveLinkHubClient* LiveLinkClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		TSubclassOf<ULiveLinkRole> LiveLinkRole = LiveLinkClient->GetSubjectRole_AnyThread(SubjectKey);
		
		if (const FLiveLinkStaticDataStruct* StaticData = LiveLinkClient->GetSubjectStaticData_AnyThread(SubjectKey, /* bGetOverrideData */ false))
		{
			if (StaticData->IsValid())
			{
				TSharedPtr<FInstancedStruct> StaticDataInstancedStruct = MakeShared<FInstancedStruct>();
				StaticDataInstancedStruct->InitializeAs(StaticData->GetStruct(), (uint8*)StaticData->GetBaseData());

				StaticDataContainer = FLiveLinkRecordingStaticDataContainer();
				StaticDataContainer->Role = LiveLinkRole;
				StaticDataContainer->RecordedData.Insert(MoveTemp(StaticDataInstancedStruct), 0);
				StaticDataContainer->Timestamps.Add(0.0);
			}
		}

		return StaticDataContainer;
	}
}

void FLiveLinkUAssetRecorder::StartRecording()
{
	check(!CurrentRecording.IsValid());
	CurrentRecording = MakePimpl<FLiveLinkUAssetRecordingData>();
	RecordInitialStaticData();

	bIsRecording = true;
	TimeRecordingStarted = FPlatformTime::Seconds();
}

void FLiveLinkUAssetRecorder::StopRecording()
{
	if (CurrentRecording)
	{
		bIsRecording = false;

		TimeRecordingEnded = FPlatformTime::Seconds();

		const double RecordingLength = TimeRecordingEnded - TimeRecordingStarted;
		SaveRecording(MoveTemp(*CurrentRecording), RecordingLength, false);
		CurrentRecording.Reset();
	}
}

bool FLiveLinkUAssetRecorder::IsRecording() const
{
	return bIsRecording;
}

bool FLiveLinkUAssetRecorder::CanRecord(FText* OutErrorMessage) const
{
	FText ErrorText;
	
	ON_SCOPE_EXIT
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = MoveTemp(ErrorText);
		}
	};
	
	const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
	if (LiveLinkHubModule.GetPlaybackController()->IsInPlayback())
	{
		return false;
	}
	
	const ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();
	if (SessionInfo.GetSessionName().IsEmpty())
	{
		ErrorText = LOCTEXT("SessionEmptyTooltip", "Session cannot be empty");
		return false;
	}

	if (!FPaths::ValidatePath(SessionInfo.GetSessionName(), &ErrorText))
	{
		ErrorText = FText::Format(LOCTEXT("SessionErrorTooltip", "Session error: {0}"), ErrorText);
		return false;
	}

	if (SessionInfo.GetSlateName().IsEmpty())
	{
		ErrorText = LOCTEXT("SlateEmptyTooltip", "Slate cannot be empty");
		return false;
	}

	if (!FPaths::ValidatePath(SessionInfo.GetSlateName(), &ErrorText))
	{
		ErrorText = FText::Format(LOCTEXT("SlateErrorTooltip", "Slate error: {0}"), ErrorText);
		return false;
	}
	
	// TODO: Check for recording destination asset naming conflict.
	return true;
}

void FLiveLinkUAssetRecorder::SaveFromExistingRecording(ULiveLinkRecording* InRecording, const FString& InFilePath, bool bUpgradeInPlace)
{
	ULiveLinkUAssetRecording* UAssetRecording = CastChecked<ULiveLinkUAssetRecording>(InRecording);

	if (bUpgradeInPlace)
	{
		SaveRecordingInPlace(UAssetRecording);
	}
	else
	{
		SaveRecording(MoveTemp(UAssetRecording->RecordingData), InRecording->LengthInSeconds, /*bIncrementTake*/ false,
			InRecording->RecordingPreset, &InRecording->FrameRate, InFilePath);
	}

	CurrentRecording.Reset();
}

bool FLiveLinkUAssetRecorder::GetSavePresetPackageName(FString& OutName, const FString& InDefaultName, bool bAlwaysOpenSaveAsDialog) const
{
	// Determine the recording mount point (custom or default /Game)
	const FString RecordingMountPoint = UE::AssetRecorderUtils::Private::GetRecordingMountPoint();

	const FDateTime Today = FDateTime::Now();

	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(TEXT("date"), Today.ToString());

	const TObjectPtr<ULiveLinkHubNamingTokens> Tokens = GetDefault<ULiveLinkHubSettings>()->GetNamingTokens();
	FNamingTokenFilterArgs Filter;
	Filter.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	check(GEditor);
	FNamingTokenResultData TemplateData = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->EvaluateTokenString(GetDefault<ULiveLinkHubSettings>()->FilenameTemplate,
		Filter);

	FString DefaultName = InDefaultName;
	FString DefaultFolder;
	// Split folder path and file name.
	if (DefaultName.IsEmpty())
	{
		FString FormattedString = TemplateData.EvaluatedText.ToString();
		int32 LastSlashIndex;
		if (FormattedString.FindLastChar('/', LastSlashIndex))
		{
			DefaultFolder = FormattedString.Left(LastSlashIndex);
			if (DefaultFolder.StartsWith("/"))
			{
				DefaultFolder.RemoveFromStart("/");
			}
			DefaultName = FormattedString.Mid(LastSlashIndex + 1);
		}
		else
		{
			DefaultName = FormattedString;
		}
	}
	
	FString ContentDir;

	ensure(FPackageName::TryConvertLongPackageNameToFilename(RecordingMountPoint, ContentDir));
	const FString DialogStartPath = FPaths::Combine(RecordingMountPoint, DefaultFolder);
	const FString AbsoluteFolderPath = FPaths::Combine(ContentDir, DefaultFolder);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// Create directory if it doesn't exist
	if (!PlatformFile.DirectoryExists(*AbsoluteFolderPath))
	{
		if (!PlatformFile.CreateDirectoryTree(*AbsoluteFolderPath))
		{
			UE_LOGF(LogLiveLinkHub, Error, "Failed to create directory %ls.", *AbsoluteFolderPath);
			return false;
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().ScanPathsSynchronous({ RecordingMountPoint }, true);
	}
	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DefaultFolder / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	const FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	bool bPromptSaveAs = bAlwaysOpenSaveAsDialog || GetDefault<ULiveLinkHubSettings>()->bPromptSaveAsOnRecord;
	// this isnt valid, check if file valid with the asset registry
	if (bPromptSaveAs || FPaths::FileExists(AbsoluteFolderPath / UniqueAssetName + TEXT(".uasset")))
	{
		// Get destination for asset
		bool bFilenameValid = false;
		while (!bFilenameValid)
		{
			if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
			{
				return false;
			}

			NewPackageName = FString::Format(*UserPackageName, FormatArgs);

			{
				// Ensure we aren't saving over an existing recording as this isn't supported yet... the package isn't saved properly.
				// todo: We should support saving over/deleting existing recordings. This is simply a low risk fix to a playback crash for 5.5.0.
				AssetToolsModule.Get().CreateUniqueAssetName(NewPackageName, TEXT(""), NewPackageName, UniqueAssetName);
			}

			FText OutError;
			bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
		}
	}
	else
	{
		NewPackageName = RecordingMountPoint / UniqueAssetName;
	}

	OutName = MoveTemp(NewPackageName);
	return true;
}

void FLiveLinkUAssetRecorder::RecordBaseData(FLiveLinkRecordingBaseDataContainer& StaticDataContainer, TSharedPtr<FInstancedStruct>&& DataToRecord)
{
	const double TimeNowInSeconds = FPlatformTime::Seconds();
	StaticDataContainer.RecordedData.Add(MoveTemp(DataToRecord));
	StaticDataContainer.Timestamps.Add(TimeNowInSeconds - TimeRecordingStarted);
}

void FLiveLinkUAssetRecorder::RecordStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData)
{
	if (bIsRecording && CurrentRecording)
	{
		FLiveLinkRecordingStaticDataContainer& StaticDataContainer = CurrentRecording->StaticData.FindOrAdd(SubjectKey);
		TSharedPtr<FInstancedStruct> NewData = MakeShared<FInstancedStruct>();
		NewData->InitializeAs(StaticData.GetStruct(), (uint8*)StaticData.GetBaseData());
		StaticDataContainer.Role = Role;

		RecordBaseData(StaticDataContainer, MoveTemp(NewData));
	}
}

void FLiveLinkUAssetRecorder::RecordFrameData(const FLiveLinkSubjectKey& SubjectKey, const FLiveLinkFrameDataStruct& FrameData)
{
	if (bIsRecording && CurrentRecording)
	{
		FLiveLinkRecordingBaseDataContainer& FrameDataContainer = CurrentRecording->FrameData.FindOrAdd(SubjectKey);
		TSharedPtr<FInstancedStruct> NewData = MakeShared<FInstancedStruct>();
		NewData->InitializeAs(FrameData.GetStruct(), (uint8*)FrameData.GetBaseData());

		RecordBaseData(FrameDataContainer, MoveTemp(NewData));
	}
}

bool FLiveLinkUAssetRecorder::OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName) const
{
	TSharedRef<SWindow> RootWindow = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkHub()->GetRootWindow();

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(ULiveLinkRecording::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveLiveLinkRecordingDialogTitle", "Save Live Link Recording");
		SaveAssetDialogConfig.WindowOverride = RootWindow;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}

void FLiveLinkUAssetRecorder::SaveRecording(FLiveLinkUAssetRecordingData&& InRecordingData,
	double InRecordingLength, bool bIncrementTake, const TObjectPtr<ULiveLinkPreset>& InPreset,
	const FFrameRate* InFrameRate, const FString& InFilePath)
{
	FString PackageName = InFilePath;
	if (PackageName.IsEmpty() && !GetSavePresetPackageName(PackageName))
	{
		return;
	}

	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* Package = FindPackage(NULL, *PackageName);
	ULiveLinkUAssetRecording* Recording = nullptr;

	Package = CreatePackage(*PackageName);
	Recording = NewObject<ULiveLinkUAssetRecording>(Package, *NewAssetName, RF_Public | RF_Standalone);

	if (Recording)
	{
		Recording->InitializeNewRecordingData(MoveTemp(InRecordingData), InRecordingLength);
		// If preset or frame rate provided make sure to set them after initializing new data.
		if (InPreset)
		{
			// Copy the provided preset to our preset. This allows full source names to be preserved if we're saving data from another recording,
			// such as during an upgrade.
			UEngine::CopyPropertiesForUnrelatedObjects(InPreset, Recording->RecordingPreset);
		}

		if (InFrameRate)
		{
			Recording->FrameRate = *InFrameRate;
		}

		Recording->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(Recording);

		const TStrongObjectPtr<ULiveLinkUAssetRecording> NewRecordingStrongPtr(Recording);
		
		// Create a task to run on a separate thread for saving all frame data and writing the final UAsset to disk.
		// We use a container rather than just one task on the chance a save operation is still running when another recording is being saved.
		FAsyncTask<FLiveLinkSaveRecordingAsyncTask>& AsyncTask = *AsyncSaveTasks.Add(NewRecordingStrongPtr,
			MakeUnique<FAsyncTask<FLiveLinkSaveRecordingAsyncTask>>(Recording, this));
		AsyncTask.StartBackgroundTask();
	}

	// Increment take on successful recording.
	if (bIncrementTake)
	{
		ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();
		SessionInfo.SetTakeNumber(SessionInfo.GetTakeNumber() + 1);
	}
}

void FLiveLinkUAssetRecorder::SaveRecordingInPlace(ULiveLinkUAssetRecording* InRecording)
{
	if (!InRecording)
	{
		return;
	}

	InRecording->SavedRecordingVersion = UE::LiveLinkHub::Private::RecordingVersions::Latest;
	InRecording->MarkPackageDirty();

	const TStrongObjectPtr<ULiveLinkUAssetRecording> NewRecordingStrongPtr(InRecording);

	// Create a task to run on a separate thread for saving all frame data and writing the final UAsset to disk.
	// We use a container rather than just one task on the chance a save operation is still running when another recording is being saved.
	FAsyncTask<FLiveLinkSaveRecordingAsyncTask>& AsyncTask = *AsyncSaveTasks.Add(NewRecordingStrongPtr,
		MakeUnique<FAsyncTask<FLiveLinkSaveRecordingAsyncTask>>(InRecording, this));
	AsyncTask.StartBackgroundTask();
}

void FLiveLinkUAssetRecorder::RecordInitialStaticData()
{
	FLiveLinkHubClient* LiveLinkClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
	TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient->GetSubjects(true, true);

	for (const FLiveLinkSubjectKey& Subject : Subjects)
	{
		TOptional<FLiveLinkRecordingStaticDataContainer> StaticDataContainer = UE::AssetRecorderUtils::Private::CreateStaticDataContainerFromFrameData(Subject);
		if (StaticDataContainer)
		{
			CurrentRecording->StaticData.Add(Subject, MoveTemp(*StaticDataContainer));
		}
	}
}

void FLiveLinkUAssetRecorder::OnRecordingDataSaved_GameThread(FLiveLinkSaveRecordingAsyncTask* InTask)
{
	const TStrongObjectPtr<ULiveLinkUAssetRecording> Recording = InTask->GetRecording().Pin();
	if (ensure(Recording.IsValid()))
	{
		FSavePackageArgs SavePackageArgs;
		SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
		SavePackageArgs.Error = GLog;
		SavePackageArgs.SaveFlags = SAVE_Async;

		const FString PackageFileName = FPackageName::LongPackageNameToFilename(Recording->GetPackage()->GetName(), FPackageName::GetAssetPackageExtension());
		if (!UPackage::SavePackage(Recording->GetPackage(), Recording.Get(), *PackageFileName, MoveTemp(SavePackageArgs)))
		{
			UE_LOGF(LogLiveLinkHub, Error, "Package '%ls' was not saved", *PackageFileName);
		}
	}

	InTask->NotifyPackageSaveStarted();
}

void FLiveLinkUAssetRecorder::OnRecordingSaveThreadFinished_GameThread(FLiveLinkSaveRecordingAsyncTask* InTask)
{
	// Make sure we see the saved file on disk in the asset registry.
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const FString RecordingMountPoint = UE::AssetRecorderUtils::Private::GetRecordingMountPoint();
	AssetRegistryModule.Get().ScanPathsSynchronous({ RecordingMountPoint }, true);
	
	TStrongObjectPtr<ULiveLinkUAssetRecording> Recording = InTask->GetRecording().Pin();
	if (ensure(Recording.IsValid()))
	{
		// Finish task first to make sure strong reference to the recording is cleared.
		if (const TUniquePtr<FAsyncTask<FLiveLinkSaveRecordingAsyncTask>>* AsyncTask = AsyncSaveTasks.Find(Recording))
		{
			AsyncTask->Get()->EnsureCompletion();
			ensure(AsyncSaveTasks.Remove(Recording) > 0);
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Error, "Could not find save task for recording: '%ls'", *Recording->GetName());
		}
	
		const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		const TStrongObjectPtr<ULiveLinkRecording> PlaybackRecording = LiveLinkHubModule.GetPlaybackController()->GetRecording();

		UPackage* PackageToUnload = Recording->GetPackage();
		const bool bIsPlayingThisRecording = PlaybackRecording.Get() == Recording.Get();

		if (!bIsPlayingThisRecording)
		{
			// Unload as this is not used again until the user loads it, and allows the bulk animation data to obtain a file handle correctly.
			Recording.Reset();
			constexpr bool bUnloadNextTick = true;
			LiveLinkHubModule.GetPlaybackController()->UnloadRecordingPackage(PackageToUnload, bUnloadNextTick);
		}
	}
}

void FLiveLinkUAssetRecorder::FLiveLinkSaveRecordingAsyncTask::DoWork()
{
	check(LiveLinkRecording.IsValid());

	// Write to bulk data.
	LiveLinkRecording->SaveRecordingData();

	// Let the game thread start the actual package save. Editor targets will assert if SavePackage is called from another thread.
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateRaw(Recorder, &FLiveLinkUAssetRecorder::OnRecordingDataSaved_GameThread, this),
	TStatId(),
	nullptr,
	ENamedThreads::GameThread);

	// Wait for the game thread to signal it has started saving the package.
	constexpr uint32 TimeoutMillis = 5000;
	if (!PackageSaveStartedEvent->Wait(TimeoutMillis))
	{
		UE_LOGF(LogLiveLinkHub, Error, "Timed out waiting for package save.");
	}

	// Block the async thread until the package saves. We wait because the engine doesn't provide any proper callback
	// when the package finishes saving async, and we still have some cleanup to do once it finishes saving.
	UPackage::WaitForAsyncFileWrites();

	// Cleanup and finish the thread.
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
	FSimpleDelegateGraphTask::FDelegate::CreateRaw(Recorder, &FLiveLinkUAssetRecorder::OnRecordingSaveThreadFinished_GameThread, this),
	TStatId(),
	nullptr,
	ENamedThreads::GameThread);
}

#undef LOCTEXT_NAMESPACE
