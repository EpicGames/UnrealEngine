// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "LiveLinkTypes.h"
#include "LiveLinkUAssetRecording.h"
#include "Misc/CoreMiscDefines.h"
#include "Recording/LiveLinkRecorder.h"
#include "Templates/PimplPtr.h"

struct FLiveLinkRecordingBaseDataContainer;
struct FLiveLinkUAssetRecordingData;
struct FInstancedStruct;

/** UAsset implementation for serializing recorded livelink data. */
class FLiveLinkUAssetRecorder : public ILiveLinkRecorder
{
public:

	//~ Begin ILiveLinkRecorder
	virtual void StartRecording() override;
	virtual void StopRecording() override;
	virtual void RecordStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData) override;
	virtual void RecordFrameData(const FLiveLinkSubjectKey& SubjectKey, const FLiveLinkFrameDataStruct& FrameData) override;
	virtual bool IsRecording() const override;
	virtual bool CanRecord(FText* OutErrorMessage) const override;
	virtual void SaveFromExistingRecording(ULiveLinkRecording* InRecording, const FString& InFilePath, bool bUpgradeInPlace) override;
	virtual bool GetSavePresetPackageName(FString& OutName, const FString& InDefaultName = FString(), bool bAlwaysOpenSaveAsDialog = false) const override;
	//~ End ILiveLinkRecorder

private:
	/** Prompt the user for a destination path for the recording. */
	bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName) const;
	/** Create a recording package and save it. */
	void SaveRecording(FLiveLinkUAssetRecordingData&& InRecordingData, double InRecordingLength,
		bool bIncrementTake, const TObjectPtr<ULiveLinkPreset>& InPreset = nullptr, const FFrameRate* InFrameRate = nullptr,
		const FString& InFilePath = FString());
	void SaveRecordingInPlace(ULiveLinkUAssetRecording* InRecording);
	/** Record data to a ULiveLinkRecording object. */
	void RecordBaseData(FLiveLinkRecordingBaseDataContainer& StaticDataContainer, TSharedPtr<FInstancedStruct>&& DataToRecord);
	/** Record initial data for all livelink subjects. (Useful when static data was sent before the recording started). */
	void RecordInitialStaticData();
	
private:
	class FLiveLinkSaveRecordingAsyncTask : public FNonAbandonableTask
	{
	public:
		FLiveLinkSaveRecordingAsyncTask(ULiveLinkUAssetRecording* InLiveLinkRecording, FLiveLinkUAssetRecorder* InRecorder)
		{
			LiveLinkRecording = InLiveLinkRecording;
			Recorder = InRecorder;
		}

		TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(LiveLinkSaveRecordingAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
		}

		void DoWork();
		
		/** Notify we have started saving the package. */
		void NotifyPackageSaveStarted() { PackageSaveStartedEvent->Trigger(); }
		/** The recording being saved by this task. */
		TWeakObjectPtr<ULiveLinkUAssetRecording> GetRecording() const { return LiveLinkRecording; }

	private:
		/** The recording being saved. */
		TWeakObjectPtr<ULiveLinkUAssetRecording> LiveLinkRecording;
		/** The recorder owner. */
		FLiveLinkUAssetRecorder* Recorder = nullptr;
		/** If the game thread has started to save the package. */
		FEventRef PackageSaveStartedEvent;
	};

	/** Called on the game thread after the recording data has been saved. */
	void OnRecordingDataSaved_GameThread(FLiveLinkSaveRecordingAsyncTask* InTask);
	/** Called when the async save thread has finished. */
	void OnRecordingSaveThreadFinished_GameThread(FLiveLinkSaveRecordingAsyncTask* InTask);
	
	/** Current async save tasks. */
	TMap<TStrongObjectPtr<ULiveLinkUAssetRecording>, TUniquePtr<FAsyncTask<FLiveLinkSaveRecordingAsyncTask>>> AsyncSaveTasks;
	/** Holds metadata and recording data. */
	TPimplPtr<FLiveLinkUAssetRecordingData> CurrentRecording;
	/** Whether we're currently recording livelink data. */
	bool bIsRecording = false;
	/** Timestamp in seconds of when the recording was started. */
	double TimeRecordingStarted = 0.0;
	/** Timestamp in seconds of when the recording ended. */
	double TimeRecordingEnded = 0.0;
};
