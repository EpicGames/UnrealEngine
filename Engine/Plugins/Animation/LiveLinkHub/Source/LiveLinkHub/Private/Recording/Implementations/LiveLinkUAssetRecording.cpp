// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkUAssetRecording.h"

#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "LiveLinkCompression.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkUAssetRecordingPlayer.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "Recording/LiveLinkRecordingRangeHelpers.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkUAssetRecording)

#define LIVELINKHUB_FRAME_DEBUG 0

namespace UE::LiveLinkHub::Compression
{
	static const FName CompressionAlgo = NAME_Oodle;

	/** Compress data with a bias for memory. */
	void Compress(TArray<uint8>& InBytes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::Compress);

		TArray<uint8> OutCompressedData;
		ECompressionFlags CompressFlags = COMPRESS_BiasMemory;
		int32 CompressedSize = FCompression::CompressMemoryBound(CompressionAlgo, InBytes.Num(), CompressFlags);
		OutCompressedData.SetNumUninitialized(CompressedSize);

		if (FCompression::CompressMemory(CompressionAlgo, OutCompressedData.GetData(), CompressedSize, InBytes.GetData(), InBytes.Num(), CompressFlags))
		{
			OutCompressedData.SetNum(CompressedSize, EAllowShrinking::No);
			InBytes = MoveTemp(OutCompressedData);
		}
		else
		{
			ensureAlways(false);
		}
	}

	/** Decompress data. */
	TArray<uint8> Decompress(FName InCompressionMethod, TArray<uint8>& InBytes, int32 UncompressedSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::Decompress);

		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(UncompressedSize);

		ensure(FCompression::UncompressMemory(InCompressionMethod, UncompressedData.GetData(), UncompressedData.Num(), InBytes.GetData(), InBytes.Num(), COMPRESS_BiasMemory));
		return UncompressedData;
	}
}

ULiveLinkUAssetRecording::~ULiveLinkUAssetRecording()
{
	if (!IsTemplate())
	{
		if (!IsEngineExitRequested())
		{
			UnloadRecordingData();
		}
		else
		{
			bCancelStream = true;
			if (AsyncStreamTask.IsValid())
			{
				if (!AsyncStreamTask->Cancel())
				{
					AsyncStreamTask->EnsureCompletion();
				}
				AsyncStreamTask.Reset();
			}
		}
	}
}

void ULiveLinkUAssetRecording::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	AnimationData.Serialize(Ar, this);
}

void ULiveLinkUAssetRecording::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		EjectAndUnload();
	}
}

void ULiveLinkUAssetRecording::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	EjectAndUnload();
}

FQualifiedFrameTime ULiveLinkUAssetRecording::GetLastFrameTime() const
{
	const FFrameRate Rate = GetGlobalFrameRate();
	const FFrameTime FrameTime = Rate.AsFrameTime(GetLastTimestamp());
	const FQualifiedFrameTime QFrameTime(FrameTime, Rate);
	return QFrameTime;
}

FFrameRate ULiveLinkUAssetRecording::GetGlobalFrameRate() const
{
	return RecordingLastTimestampFramerate.IsValid() ? RecordingLastTimestampFramerate : FFrameRate(60, 1);
}

void ULiveLinkUAssetRecording::LoadEntireRecording()
{
	auto LoadEntireRecordingLambda = [this]()
	{
		const int32 SavedMaxFrames = GetMaxFrames();
		if (SavedMaxFrames <= 0)
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not load the recording because the max frame size is 0.");
			return;
		}
		// Start from a clean state.
		UnloadRecordingData();

		bIsPerformingFullLoad = true;

		LoadRecordingData(0, SavedMaxFrames);	
	};
	
	if (!bPerformedInitialLoad)
	{
		GetOnRecordingInitialLoad().BindLambda(LoadEntireRecordingLambda);

		// Loads max frames.
		LoadRecordingData(0, 1);
	}
	else
	{
		LoadEntireRecordingLambda();
	}
}

void ULiveLinkUAssetRecording::SaveRecordingData()
{
	bIsSavingRecordingData = true;
	
	TArray64<uint8> Memory;
	FMemoryWriter64 Archive(Memory);
	
	int32 RecordingVersionToSave = CurrentRecordingVersion;
	Archive << RecordingVersionToSave;

	// How much static data to expect.
	int32 NumStaticData = RecordingData.StaticData.Num();
	Archive << NumStaticData;
		
	for (TTuple<FLiveLinkSubjectKey, FLiveLinkRecordingStaticDataContainer>& StaticData : RecordingData.StaticData)
	{
		SaveFrameData(&Archive, StaticData.Key, StaticData.Value);
	}
	
	// How much frame data to expect.
	int32 NumFrameData = RecordingData.FrameData.Num();
	Archive << NumFrameData;
		
	for (TTuple<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer>& FrameData : RecordingData.FrameData)
	{
		SaveFrameData(&Archive, FrameData.Key, FrameData.Value);
	}

	AnimationData.WriteBulkData(Memory);

	bIsSavingRecordingData = false;
}

void ULiveLinkUAssetRecording::LoadRecordingData(int32 InInitialFrame, int32 InNumFramesToLoad)
{
	if (bIsFullyLoaded)
	{
		return;
	}
	
	bCancelStream = false;
	bPauseStream = false;
	OnStreamPausedEvent->Reset();
	OnStreamUnpausedEvent->Reset();
	
	int32 StartFrame = InInitialFrame - InNumFramesToLoad;
	if (StartFrame < 0)
	{
		StartFrame = 0;
	}
	
	// Additional buffer to each side, plus the initial frame.
	InNumFramesToLoad = (InNumFramesToLoad * 2) + 1;

	// Perform initial setup of the file reader.
	if (!AsyncStreamTask.IsValid())
	{
		FrameFileData.Empty();
	}

	// Stream more to the left if we're nearing the end of the recording.
	const int32 FramesPastLimit = FMath::Max(0, (StartFrame + InNumFramesToLoad) - GetMaxFrames());
	EarliestFrameToStream = StartFrame - FramesPastLimit;

	if (InitialFrameToStream != InInitialFrame)
	{
		// Signal to the async thread we're changing the desired frames, so it will restart itself using
		// the most up-to-date requested frame. 
		StreamingFrameChangeFromFrame = InitialFrameToStream;
		UE_LOGF(LogLiveLinkHub, Verbose, "Stream initial frame changed from: %d to %d ", InInitialFrame, InitialFrameToStream);
	}
	
	InitialFrameToStream = InInitialFrame;
	TotalFramesToStream = InNumFramesToLoad;

	if (!OnPreGarbageCollectHandle.IsValid())
	{
		OnPreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &ULiveLinkUAssetRecording::OnPreGarbageCollect);
	}

	if (!OnPostGarbageCollectHandle.IsValid())
	{
		OnPostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &ULiveLinkUAssetRecording::OnPostGarbageCollect);
	}
	
	if (!AsyncStreamTask.IsValid())
	{
		AsyncStreamTask = MakeUnique<FAsyncTask<FLiveLinkStreamAsyncTask>>(this);
		AsyncStreamTask->StartBackgroundTask();
	}
}

void ULiveLinkUAssetRecording::LoadEntireRecordingForUpgrade()
{
	bIsUpgrading = true;
	LoadEntireRecording();
}

void ULiveLinkUAssetRecording::UnloadRecordingData()
{
	// We need to prevent unloading if a package is being saved, but if this is called in a case where there is no outer,
	// the engine will CastCheck to find the package and fail. We need to avoid in this scenario since that would imply
	// no package is being saved. This was reported being triggered during an editor shutdown under certain conditions.
	const UPackage* Package = (GetOuter() == nullptr) ? nullptr : GetPackage();
	if (IsSavingRecordingData() || (Package && Package->HasAnyPackageFlags(PKG_IsSaving)))
	{
		UE_LOGF(LogLiveLinkHub, Warning, "Attempted to unload %ls while the package was still being saved", *GetName());
		return;
	}

	bIsPerformingFullLoad = false;
	
	bCancelStream = true;
	UnpauseStream();
	bIsFullyLoaded = false;

	if (AsyncStreamTask.IsValid())
	{
		if (!AsyncStreamTask->Cancel())
		{
			AsyncStreamTask->EnsureCompletion();
		}
		AsyncStreamTask.Reset();
	}

	if (OnPreGarbageCollectHandle.IsValid())
	{
		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(OnPreGarbageCollectHandle);
		OnPreGarbageCollectHandle.Reset();
	}

	if (OnPostGarbageCollectHandle.IsValid())
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(OnPostGarbageCollectHandle);
		OnPostGarbageCollectHandle.Reset();
	}

	bPerformedInitialLoad = false;

	AnimationData.UnloadBulkData();

	FrameFileData.Empty();
	RecordingMaxFrames = 0;
	RecordingLastTimestamp = 0.0;
	MaxFrameDiskSize = 0;
	EarliestFrameToStream = 0;
	InitialFrameToStream = 0;
	TotalFramesToStream = 0;
	
	for (TTuple<FLiveLinkSubjectKey, FLiveLinkRecordingStaticDataContainer>& StaticData : RecordingData.StaticData)
	{
		StaticData.Value.Timestamps.Empty();
		StaticData.Value.RecordedData.Empty();
	}

	for (TTuple<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer>& FrameData : RecordingData.FrameData)
	{
		FrameData.Value.Timestamps.Empty();
		FrameData.Value.RecordedData.Empty();
	}
}

bool ULiveLinkUAssetRecording::WaitForBufferedFrames(int32 InMinFrame, int32 InMaxFrame)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::WaitForBufferedFrames"), STAT_ULiveLinkUAssetRecording_WaitForBufferedFrames, STATGROUP_LiveLinkHub);
	
	if (AsyncStreamTask.IsValid())
	{
		// Max frames isn't set until after the initial load.
		while (!bPerformedInitialLoad)
		{
			if (bCancelStream)
			{
				// Likely encountered an error.
				UnloadRecordingData();
				return false;
			}
			FPlatformProcess::Sleep(0.002f);
		}

		const int32 EndFrame = FMath::Clamp(GetLastFrameTime().Time.RoundToFrame().Value, 0, RecordingMaxFrames - 1);
		if (EndFrame <= 0)
		{
			return true;
		}
		
		// Clamp the frame range to the max possible range. If the selection range extends the actual frame range
		// then there would be nothing to load.
		InMinFrame = FMath::Clamp(InMinFrame, 0, EndFrame);
		InMaxFrame = FMath::Clamp(InMaxFrame, 0, EndFrame);

		const int32 InTotalFrames = InMaxFrame - InMinFrame + 1;
		const TRange<int32> InRange = UE::LiveLinkHub::RangeHelpers::Private::MakeInclusiveRange(InMinFrame, InMaxFrame);

		while (true)
		{
			if (InTotalFrames > TotalFramesToStream
			|| IsFrameRangeBuffered(InRange)
			|| AsyncStreamTask->IsDone())
			{
				break;
			}
			// We could potentially reduce blocking operations here by leveraging StreamingFrameChangeFromFrame and returning false
			// if it has changed. Doing so would require both allowing it to change at this point (this is all done from the game thread)
			// and another mechanism for ensuring the requested frame(s) we are blocking for are still delivered once loaded.
			
			FPlatformProcess::Sleep(0.002f);
		}
	}

	return true;
}

UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> ULiveLinkUAssetRecording::GetBufferedFrameRanges(bool bIncludeInactive) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::GetBufferedFrameRanges"), STAT_ULiveLinkUAssetRecording_GetBufferedFrameRanges, STATGROUP_LiveLinkHub);
	
	if (bIsFullyLoaded)
	{
		return { UE::LiveLinkHub::RangeHelpers::Private::MakeInclusiveRange(0, RecordingMaxFrames - 1) };
	}

	FScopeLock Lock(&BufferedFrameMutex);
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> Result = BufferedFrameRanges;
	if (bIncludeInactive)
	{
		Result.Append(InactiveBufferedFrameRanges);
	}
	return Result;
}

bool ULiveLinkUAssetRecording::IsFrameRangeBuffered(const TRange<int32>& InRange) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::IsFrameRangeBuffered"), STAT_ULiveLinkUAssetRecording_IsFrameRangeBuffered, STATGROUP_LiveLinkHub);
    	
	TRangeSet<int32> Union;
	
	const UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> Ranges = GetBufferedFrameRanges(/*bIncludeInactive*/ false);
	for (const TRange<int32>& Range : Ranges)
	{
		if (!Range.IsEmpty())
		{
			Union.Add(Range);
		}
	}

	// Quick verification the buffered range contains our desired range. 
	if (Union.Contains(InRange))
	{
		// -------------------------------------------------------------------------------------------------------------------------------------
		// The buffered range is only exact across all tracks the initial time frames are loaded, since the range is only updated once a given
		// frame is buffered for all tracks. Once we move the scrubber position, it's no longer completely accurate, as one track might have some
		// frames unloaded, and another track might have moved frames to a cache. This depends on buffer size, track frame rate, etc.

		// We can run into issues when one track has frames loaded for the range size, but another track no longer does. Without additional checks
		// this method won't correctly report if the range is really buffered, causing the wrong frame to be displayed for certain tracks.

		// Rather than check if every frame in the range is truly loaded across all tracks, we can just check if the target frame is. This
		// method is only used to wait for loaded frames and gets updated when the scrubber position changes, so this single check should
		// always ensure the frame we are displaying across all tracks is correct while being reasonably efficient.
		// -------------------------------------------------------------------------------------------------------------------------------------
		
		// Check the target frame we're landing on is actually ready. The lower bound value is the exact
		// frame we are scrubbing to, so it must be ready on all tracks.
		const FQualifiedFrameTime TargetFrame(FFrameTime(InRange.GetLowerBoundValue()), GetGlobalFrameRate());
		const double TargetFrameTime = TargetFrame.AsSeconds();
		const int32 FrameIdx = TargetFrame.Time.GetFrame().Value;
		
		using namespace UE::LiveLinkHub::FrameData::Private;

		// Iterate each frame data and verify the target frame is loaded and ready.
		for (const TPair<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer>& Pair : RecordingData.FrameData)
		{
			// Quickly check the frame idx to see if it is loaded for the track. O(1)
			if (!Pair.Value.IsFrameLoaded(FrameIdx))
			{
				// If the frame is not loaded, but we've passed the buffer check above, then there is at least one other track
				// that has this frame loaded. What might be happening is this particular track has different frame dispersion.

				// O(1)
				const FFrameMetaData& MetaData = FrameFileData.FindChecked(Pair.Key);

				// Find the closest frame to the target frame: O(log n)
				// This frame is either approximately close, or it could be farther in the past or future depending on the track's frames.
				// Either way, this frame is required and will be loaded next if it isn't already.
				const int32 LocalTargetFrameHint = TargetFrame.ConvertTo(MetaData.LocalFrameRate).FrameNumber.Value;
				const int32 ClosestFrameIdx = MetaData.FindFrameFromTimestamp(TargetFrameTime, LocalTargetFrameHint);
				if (!Pair.Value.IsFrameLoaded(ClosestFrameIdx))
				{
					return false;
				}
			}
		}
	
		return true;
	}
	
	return false;
}

void ULiveLinkUAssetRecording::CopyRecordingData(FLiveLinkPlaybackTracks& InOutLiveLinkPlaybackTracks) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::CopyRecordingData"), STAT_ULiveLinkUAssetRecording_CopyRecordingData, STATGROUP_LiveLinkHub);
	
	FScopeLock Lock(&DataContainerMutex);

	for (const TPair<FLiveLinkSubjectKey, FLiveLinkRecordingStaticDataContainer>& Pair : RecordingData.StaticData)
	{
		// Modify subject name so a duplicate FLiveLinkSubjectKey below doesn't produce the same hash. This allows us to efficiently
		// reuse tracks, as well as preserve the absolute frame index, which is needed since frame data is streamed in.
		FLiveLinkSubjectKey StaticSubjectKey = Pair.Key;
		StaticSubjectKey.SubjectName.Name = *(StaticSubjectKey.SubjectName.ToString() + "_STATIC");
		FLiveLinkPlaybackTrack& PlaybackTrack = InOutLiveLinkPlaybackTracks.Tracks.FindOrAdd(StaticSubjectKey);

		PlaybackTrack.FrameData = Pair.Value.RecordedData;
		PlaybackTrack.Timestamps = TConstArrayView<double>(Pair.Value.Timestamps);
		PlaybackTrack.LiveLinkRole = Pair.Value.Role;
		PlaybackTrack.SubjectKey = Pair.Key;
		PlaybackTrack.StartIndexOffset = Pair.Value.RecordedDataStartFrame;

		// Don't need to record framerate for static data.
	}

	for (const TPair<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer>& Pair : RecordingData.FrameData)
	{
		FLiveLinkPlaybackTrack& PlaybackTrack = InOutLiveLinkPlaybackTracks.Tracks.FindOrAdd(Pair.Key);

		const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& MetaData = FrameFileData.FindChecked(Pair.Key);

		PlaybackTrack.FrameData = Pair.Value.RecordedData;
		PlaybackTrack.Timestamps = TConstArrayView<double>(Pair.Value.Timestamps);
		PlaybackTrack.SubjectKey = Pair.Key;
		PlaybackTrack.StartIndexOffset = Pair.Value.RecordedDataStartFrame;
		PlaybackTrack.LocalFrameRate = MetaData.LocalFrameRate;
	}
}

void ULiveLinkUAssetRecording::InitializeNewRecordingData(FLiveLinkUAssetRecordingData&& InRecordingData, double InRecordingLengthSeconds)
{
	RecordingData = MoveTemp(InRecordingData);
	LengthInSeconds = InRecordingLengthSeconds;
	SavedRecordingVersion = CurrentRecordingVersion;
	FrameRate = FApp::GetTimecodeFrameRate();
	RecordingPreset->BuildFromClient();

	int32 MaxFrames = 0;
	double MaxLastTimestamp = 0.0;
	for (const TTuple<FLiveLinkSubjectKey, FLiveLinkRecordingBaseDataContainer>& FrameData : RecordingData.FrameData)
	{
		const int32 LocalMaxFrames = FrameData.Value.Timestamps.Num();
		if (LocalMaxFrames > MaxFrames)
		{
			MaxFrames = LocalMaxFrames;
		}

		double LocalLastTimestamp = 0.0;
		if (FrameData.Value.Timestamps.Num() > 0)
		{
			LocalLastTimestamp = FrameData.Value.Timestamps.Last();
			if (LocalLastTimestamp > MaxLastTimestamp)
			{
				MaxLastTimestamp = LocalLastTimestamp;
			}
		}

		// The metadata for a recording already loaded is relevant just for the frame rate.
		UE::LiveLinkHub::FrameData::Private::FFrameMetaData MetaData;
		{
			MetaData.LastTimestamp = LocalLastTimestamp;
			MetaData.MaxFrames = LocalMaxFrames;
			MetaData.LocalFrameRate = CalculateFrameRate(LocalMaxFrames, LocalLastTimestamp);
		}
		FrameFileData.Add(FrameData.Key, MoveTemp(MetaData));
	}

	RecordingMaxFrames = MaxFrames;
	RecordingLastTimestamp = MaxLastTimestamp;
	
	bIsFullyLoaded = true;
}

FFrameRate ULiveLinkUAssetRecording::CalculateFrameRate(const int32 InMaxFrames, const double InTime)
{
	const double FramesPerSecond = InTime > 0.0 ? InMaxFrames / InTime : 0.0;

	const double IntegralPart = FMath::FloorToDouble(FramesPerSecond);
	const double FractionalPart = FramesPerSecond - IntegralPart;

	constexpr uint32 Precision = 100000;
	const uint32 Divisor = FMath::GreatestCommonDivisor(FMath::RoundToDouble(FractionalPart * Precision), Precision);

	const uint32 Denominator = Precision / Divisor;
	const uint32 Numerator = IntegralPart * Denominator + FMath::RoundToDouble(FractionalPart * Precision) / Divisor;

	return FFrameRate(Numerator, Denominator);
}

void ULiveLinkUAssetRecording::SaveFrameData(FArchive* InFileWriter, const FLiveLinkSubjectKey& InSubjectKey, FLiveLinkRecordingBaseDataContainer& InBaseDataContainer)
{
	// This will crash if it fails -- we don't want to save invalid data.
	InBaseDataContainer.ValidateData();
		
	// Start block with map key.
	FGuid Source = InSubjectKey.Source;
	FString SubjectName = InSubjectKey.SubjectName.ToString();
	int32 NumFrames = InBaseDataContainer.RecordedData.Num();

	// We record the frame header size first, so later we can bulk load the entire block into memory, then feed it to a memory reader.
	const uint64 FrameHeaderSizePosition = InFileWriter->Tell();
	int32 FrameHeaderSize = 0;
	*InFileWriter << FrameHeaderSize;
	const int64 FrameHeaderSizeStart = InFileWriter->Tell();
	
	*InFileWriter << Source;
	*InFileWriter << SubjectName;
	*InFileWriter << NumFrames;

	if (NumFrames == 0)
	{
		UE_LOGF(LogLiveLinkHub, Error, "No frames recorded.");
		return;
	}
	
	const UScriptStruct* ScriptStruct = InBaseDataContainer.RecordedData[0]->GetScriptStruct();
	FString StructTypeName = ScriptStruct->GetPathName();

	// Write the struct name and size so it can be loaded later.
	*InFileWriter << StructTypeName;

	// Offset and size.
	TArray<TTuple<int64, int32>> SerializedFrameSizes;
	SerializedFrameSizes.AddDefaulted(NumFrames);

	// Remember the position to write the frame size.
	const uint64 SerializedFrameSizePosition = InFileWriter->Tell();
	*InFileWriter << SerializedFrameSizes;
	
	SerializedFrameSizes.Reset();

	// Store all timestamps in a block.
	*InFileWriter << InBaseDataContainer.Timestamps;

	FString CompressionMethod = UE::LiveLinkHub::Compression::CompressionAlgo.ToString();
	*InFileWriter << CompressionMethod;

	// Write the frame header size.
	{
		const uint64 CurrentPosition = InFileWriter->Tell();
		FrameHeaderSize = CurrentPosition - FrameHeaderSizeStart;

		InFileWriter->Seek(FrameHeaderSizePosition);
		*InFileWriter << FrameHeaderSize;
		InFileWriter->Seek(CurrentPosition);
	}

	int64 RelativeStartPosition = 0;

	// Memory used for the intermediate step of serializing data before compressing it.
	TArray<uint8> Backing;

	for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
	{
		TSharedPtr<FInstancedStruct>& Frame = InBaseDataContainer.RecordedData[FrameIdx];
		check(Frame.IsValid() && Frame->IsValid());

		// Beginning of the frame data.
		const int64 StartFramePosition = InFileWriter->Tell();
			
		// Write the frame index for streaming frames when loading.
		*InFileWriter << FrameIdx;
			
		// Write the entire frame data to memory
		FMemoryWriter MemoryWriter{ Backing, true };
		FObjectAndNameAsStringProxyArchive StructAr(MemoryWriter, false);

		Frame->Serialize(StructAr);

		int32 UncompressedSize = Backing.Num();
		*InFileWriter << UncompressedSize;

		// Then compress it and write it to the file. 
		// Note: We compress every frame independently to make it easier for the loading code to stream the data.
		UE::LiveLinkHub::Compression::Compress(Backing);
		InFileWriter->Serialize(Backing.GetData(), Backing.Num());

		Backing.Reset();

		// Store the serialized frame size, so we can write it once later.
		{
			const int64 EndFramePosition = InFileWriter->Tell();
			const int64 CurrentSerializedFrameSize64 = EndFramePosition - StartFramePosition;

			// Check for overflow.
			checkf(CurrentSerializedFrameSize64 >= INT32_MIN && CurrentSerializedFrameSize64 <= INT32_MAX,
				TEXT("Frame size overflow or underflow during save. Frame size: %lld (StartFramePosition: %lld, EndFramePosition: %lld)"),
				CurrentSerializedFrameSize64, StartFramePosition, EndFramePosition);

			const int32 CurrentSerializedFrameSize = static_cast<int32>(CurrentSerializedFrameSize64);
			
			SerializedFrameSizes.Add({ RelativeStartPosition, CurrentSerializedFrameSize });
			RelativeStartPosition += CurrentSerializedFrameSize;
		}
	}

	if (SerializedFrameSizes.Num() > 0)
	{
		// Write the frame data offset at the beginning of the block.
		const uint64 FinalOffset = InFileWriter->Tell();
		InFileWriter->Seek(SerializedFrameSizePosition);
		*InFileWriter << SerializedFrameSizes;
		InFileWriter->Seek(FinalOffset);
	}
}

static TAutoConsoleVariable<float> CVarLiveLinkHubDebugFrameBufferDelay(
	TEXT("LiveLinkHub.Debug.FrameBufferDelay"),
	0.0f,
	TEXT("The number of seconds to wait when buffering each frame.")
);

void ULiveLinkUAssetRecording::LoadRecordingAsync(int32 InStartFrame, int32 InCurrentFrame, int32 InNumFramesToLoad)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadRecordingAsync"), STAT_ULiveLinkUAssetRecording_LoadRecordingAsync, STATGROUP_LiveLinkHub);

	using namespace UE::LiveLinkHub;
	
	StreamingFrameChangeFromFrame = INDEX_NONE;
	
	const int32 MaxPossibleFrame = RecordingMaxFrames - 1;
	InStartFrame = FMath::Clamp(InStartFrame, 0, MaxPossibleFrame);
	InCurrentFrame = FMath::Clamp(InCurrentFrame, 0, MaxPossibleFrame);
	const int32 EndFrame = InStartFrame + InNumFramesToLoad;

	UE_LOGF(LogLiveLinkHub, Verbose, "Loading recording StartFrame: %d, CurrentFrame: %d, EndFrame: %d ", InStartFrame, InCurrentFrame, EndFrame);
	
	ON_SCOPE_EXIT
	{
		const bool bHasPerformedInitialLoad = bPerformedInitialLoad;
		// Always set to true. Some blocking operations wait for this, and in the case of a non-fatal error
		// we want to display error logs and don't want the program to freeze.
		bPerformedInitialLoad = !bCancelStream;

		if (!bHasPerformedInitialLoad && bPerformedInitialLoad)
		{
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
				FSimpleDelegateGraphTask::FDelegate::CreateLambda([WeakRecording = MakeWeakObjectPtr(this)] ()
					{
						if (WeakRecording.IsValid())
						{
							WeakRecording->GetOnRecordingInitialLoad().ExecuteIfBound();
						}
					}),
			TStatId(),
			nullptr,
			ENamedThreads::GameThread);
		}
	};
	
	// Perform initial load and record entry frame file offsets.
	const bool bInitialLoad = FrameFileData.Num() == 0;
	if (bInitialLoad)
	{
		AnimationData.ResetBulkDataOffset();
		
		int32 LoadedRecordingVersion;
		AnimationData.ReadBulkDataPrimitive(LoadedRecordingVersion);

		// If we modify the RecordingVersion we can perform import logic here.
		if (LoadedRecordingVersion != CurrentRecordingVersion)
		{
			UE_LOGF(LogLiveLinkHub, Log, "Converting version %d to %d", LoadedRecordingVersion, CurrentRecordingVersion);
		}
			
		RecordingVersionBeingLoaded = LoadedRecordingVersion;
			
		// Process static data.
		
		int32 NumStaticData = 0;
		
		AnimationData.ReadBulkDataPrimitive(NumStaticData);

		for (int32 StaticIdx = 0; StaticIdx < NumStaticData; ++StaticIdx)
		{
			// Create framedata just to load initial static frame data. Static data doesn't require this afterward.
			FrameData::Private::FFrameMetaData TemporaryFrameData;
			if (!LoadInitialFrameData(TemporaryFrameData))
			{
				bCancelStream = true;
				return;
			}

			FLiveLinkRecordingStaticDataContainer& DataContainer = RecordingData.StaticData.FindChecked(*TemporaryFrameData.FrameDataSubjectKey.Get());
			LoadFrameData(TemporaryFrameData, DataContainer, 0, 0, TemporaryFrameData.MaxFrames, /* bForceSequential */ true);

			FScopeLock Lock(&DataContainerMutex);
			MoveFrameDataToContainer(DataContainer, TemporaryFrameData);
		}

		// Process frame data.
		
		int32 NumFrameData = 0;
		AnimationData.ReadBulkDataPrimitive(NumFrameData);

		for (int32 FrameIdx = 0; FrameIdx < NumFrameData; ++FrameIdx)
		{
			FrameData::Private::FFrameMetaData KeyPosition;
			if (!LoadInitialFrameData(KeyPosition))
			{
				bCancelStream = true;
				return;
			}

			// Offset to the end of this block if there is multiple NumFrameData.
			const int64 EndBlockPosition = KeyPosition.GetFrameFilePosition(KeyPosition.MaxFrames - 1) + KeyPosition.GetFrameDiskSize(KeyPosition.MaxFrames - 1);
			AnimationData.SetBulkDataOffset(EndBlockPosition);
			FrameFileData.Add(*KeyPosition.FrameDataSubjectKey, MoveTemp(KeyPosition));
		}

		if (bIsPerformingFullLoad)
		{
			// Normally set at the end of scope, but if doing a full load, we won't hit that until after everything is loaded,
			// and we may have observers who want to know the status.
			bPerformedInitialLoad = true;
		}
	}
	
	// Load the required frames, either on initial load or subsequent loads.
	int32 CompletedTasks = 0;
	while (true)
	{
		WaitIfPaused_AsyncThread();
		
		// Break each frame data segment into its own "task" and context switch between them, by buffering up to LiveLinkHubSettings->BufferBatchSize.
		// The goal is to buffer the same frame numbers on multiple sources before reporting the frames as loaded, without delaying overall load
		// too much. This way when a frame number is requested, that frame will be ready across all sources.
		bool bHasCanceled = false;
		for (TTuple<FLiveLinkSubjectKey, FrameData::Private::FFrameMetaData>& FrameDataKeyVal : FrameFileData)
		{
			FrameData::Private::FFrameMetaData& FrameData = FrameDataKeyVal.Value;
			if (FrameData.BufferIterationData.Status <= FrameData::Private::FFrameBufferIterationData::Active)
			{
				if (ensure(FrameData.FrameDataSubjectKey.IsValid()))
				{
					FLiveLinkRecordingBaseDataContainer& DataContainer = RecordingData.FrameData.FindChecked(*FrameData.FrameDataSubjectKey.Get());
					LoadFrameData(FrameData, DataContainer, InStartFrame, InCurrentFrame, InNumFramesToLoad);
					if (FrameData.BufferIterationData.Status == FrameData::Private::FFrameBufferIterationData::Complete)
					{
						CompletedTasks++;
					}
				}
				else
				{
					UE_LOGF(LogLiveLinkHub, Error, "FrameDataSubjectKey is missing for recording %ls.", *GetName());
					CompletedTasks++;
				}
			}
			if (FrameData.BufferIterationData.Status == FrameData::Private::FFrameBufferIterationData::Canceled)
			{
				bHasCanceled = true;
				// Don't break, we still want to complete current iterations up to the batch size.
			}
		}

		const bool bCompletedAllTasks = CompletedTasks >= FrameFileData.Num() || bHasCanceled;
		
		// Make data available on the game thread.
		{
			FScopeLock Lock(&DataContainerMutex);
			for (TTuple<FLiveLinkSubjectKey, FrameData::Private::FFrameMetaData>& FrameDataKeyVal : FrameFileData)
			{
				FrameData::Private::FFrameMetaData& FrameData = FrameDataKeyVal.Value;
				MoveFrameDataToContainer(RecordingData.FrameData.FindChecked(*FrameData.FrameDataSubjectKey.Get()), FrameData);
				
				if (bCompletedAllTasks)
				{
					DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::TaskCleanup"), STAT_ULiveLinkUAssetRecording_TaskCleanup, STATGROUP_LiveLinkHub);
					FrameData.BufferedCache.CleanCache(RangeHelpers::Private::MakeInclusiveRange(
					InStartFrame,
					EndFrame
					));

					FrameData.BufferIterationData.Reset();
				}
			}

			UpdateBufferedFrames();
		}
		
		if (bCompletedAllTasks)
		{
			break;
		}
	}
}

bool ULiveLinkUAssetRecording::LoadInitialFrameData(UE::LiveLinkHub::FrameData::Private::FFrameMetaData& OutFrameData)
{
	int32 FrameHeaderSize = 0;
	AnimationData.ReadBulkDataPrimitive(FrameHeaderSize);

	{
		const TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> Reader = AnimationData.CreateBulkDataMemoryReader(FrameHeaderSize);
	
		FGuid KeySource = FGuid();
		FString KeyName;

		Reader->GetMemoryReader() << KeySource;
		Reader->GetMemoryReader() << KeyName;
			
		OutFrameData.FrameDataSubjectKey = MakeShared<FLiveLinkSubjectKey>(KeySource, *KeyName);
	
		int32 MaxFrames = 0;
		Reader->GetMemoryReader() << MaxFrames;

		if (MaxFrames > RecordingMaxFrames)
		{
			RecordingMaxFrames = MaxFrames;
		}
	
		OutFrameData.MaxFrames = MaxFrames;
			
		if (MaxFrames > 0)
		{	
			FString StructTypeName;
			TArray<TTuple<int64, int32>> SerializedFrameSizes;
			
			Reader->GetMemoryReader() << StructTypeName;

			if (RecordingVersionBeingLoaded < UE::LiveLinkHub::Private::RecordingVersions::DynamicFrameSizes)
			{
				// Convert from 5.5.0 recordings where we expected all frames to be a constant size.
				int32 SerializedFrameSize = 0;
				AnimationData.ReadBulkDataPrimitive(SerializedFrameSize);
				// Frame size consists of the frame index, timestamp, and frame struct data.
				SerializedFrameSizes.Init({ 0, sizeof(int32) + sizeof(double) + SerializedFrameSize }, MaxFrames);
			}
			else
			{
				Reader->GetMemoryReader() << SerializedFrameSizes;
			}

			OutFrameData.FrameDiskSizes = MoveTemp(SerializedFrameSizes);
			OutFrameData.RecordingStartFrameFilePosition = AnimationData.GetBulkDataOffset();

			OutFrameData.LoadedStruct = FindObject<UScriptStruct>(nullptr, *StructTypeName, EFindObjectFlags::ExactClass);
			if (!OutFrameData.LoadedStruct.IsValid())
			{
				UE_LOGF(LogLiveLinkHub, Error, "Script struct type '%ls' not found.", *StructTypeName);
				return false;
			}

			// Determine max frame size and if there are different frame sizes.
			if (OutFrameData.FrameDiskSizes.Num() > 0)
			{
				bool bIsConsistentSize = true;
				int32 MaxValue = TNumericLimits<int32>::Lowest();
				int32 LastValue = OutFrameData.FrameDiskSizes[0].Value;
				for (const TTuple<int64, int32>& Tuple : OutFrameData.FrameDiskSizes)
				{
					if (LastValue != Tuple.Value)
					{
						bIsConsistentSize = false;
					}

					LastValue = Tuple.Value;
					
					if (Tuple.Value > MaxValue)
					{
						MaxValue = Tuple.Value;
					}
				}

				OutFrameData.bHasConsistentFrameSize = bIsConsistentSize;
			
				MaxFrameDiskSize = MaxValue;
			}
		}

		const bool bUsesOldTimestamps = RecordingVersionBeingLoaded < UE::LiveLinkHub::Private::RecordingVersions::BulkTimestamps;
		// Find the last timestamp, this is so we can calculate the correct framerate for this track.
		if (bUsesOldTimestamps)
		{
			OutFrameData.Timestamps.Reserve(OutFrameData.MaxFrames);
		
			for (int32 Idx = 0; Idx < OutFrameData.MaxFrames; ++Idx)
			{
				double Timestamp = 0.0;
				if (!LoadTimestampFromDisk(Idx, OutFrameData, Timestamp))
				{
					return false;
				}

				OutFrameData.Timestamps.Add(Timestamp);
			}
		}
		else
		{
			Reader->GetMemoryReader() << OutFrameData.Timestamps;
		}

		if (RecordingVersionBeingLoaded >= UE::LiveLinkHub::Private::RecordingVersions::CompressionSupport)
		{
			FString CompressionMethodName;
			Reader->GetMemoryReader() << CompressionMethodName;

			OutFrameData.CompressionMethodName = *CompressionMethodName;
		}
	}

	OutFrameData.LastTimestamp = OutFrameData.Timestamps.Num () > 0 ? OutFrameData.Timestamps.Last() : 0.0;

	// Calculate frame rate.
	OutFrameData.LocalFrameRate = CalculateFrameRate(OutFrameData.MaxFrames, OutFrameData.LastTimestamp);

	if (OutFrameData.LastTimestamp > RecordingLastTimestamp)
	{
		RecordingLastTimestamp = OutFrameData.LastTimestamp;
		RecordingLastTimestampFramerate = OutFrameData.LocalFrameRate;

#if LIVELINKHUB_FRAME_DEBUG
		// Test consistency -- We need to be able to reliably convert to our global rate, which
		// is determined entirely by the track with the furthest timestamp. This ensures we are always able to
		// convert to the end frame, and that the end frame equals the original recorded value. Helps to make
		// sure a recording will always know when its last frame is. This helps both with looping and preventing
		// freezes waiting for frames to buffer that don't exist.
		{
			const FFrameTime FrameTime = RecordingLastTimestampFramerate.AsFrameTime(RecordingLastTimestamp);
			const FQualifiedFrameTime QFrameTime(FrameTime, RecordingLastTimestampFramerate);

			const int32 CalculatedMaxFrames = QFrameTime.Time.RoundToFrame().Value;
			const double CalculatedTimestamp = QFrameTime.AsSeconds();
			ensureAlways(CalculatedMaxFrames == OutFrameData.MaxFrames);
			ensureAlways(FMath::IsNearlyEqual(CalculatedTimestamp, OutFrameData.LastTimestamp, 0.0001));
		}
#endif
	}
	
	return true;
}

void ULiveLinkUAssetRecording::LoadFrameData(UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData, FLiveLinkRecordingBaseDataContainer& InDataContainer,
                                             int32 RequestedStartFrame, int32 RequestedInitialFrame, int32 RequestedFramesToLoad, bool bForceSequential)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameData"), STAT_ULiveLinkUAssetRecording_LoadFrameData, STATGROUP_LiveLinkHub);

	using namespace UE::LiveLinkHub;
	
	FrameData::Private::FFrameBufferIterationData& IterationData = InFrameData.BufferIterationData;
	
	const FFrameRate RecordingFrameRate = GetGlobalFrameRate();
	const FQualifiedFrameTime StartFrameTime(FFrameTime(RequestedStartFrame), RecordingFrameRate);
	const FQualifiedFrameTime StartInitialFrameTime(FFrameTime(RequestedInitialFrame), RecordingFrameRate);
	
	double RequestedStartTimestamp = StartFrameTime.AsSeconds();
	double RequestedInitialTimestamp = StartInitialFrameTime.AsSeconds();
	if (FMath::IsNaN(RequestedStartTimestamp))
	{
		RequestedStartTimestamp = 0.0;
	}
	if (FMath::IsNaN(RequestedInitialTimestamp))
	{
		RequestedInitialTimestamp = 0.0;
	}
	
	// First, localize the frame times to this frame data.
	if (InFrameData.LocalFrameRate.IsValid())
	{
		RequestedStartFrame = StartFrameTime.ConvertTo(InFrameData.LocalFrameRate).RoundToFrame().Value;
		RequestedInitialFrame = StartInitialFrameTime.ConvertTo(InFrameData.LocalFrameRate).RoundToFrame().Value;
	}

	// Localizing the frame rates may not be enough if the track had inconsistent frames or frame rate. Next, we
	// need to properly identify the exact local frame to use most closely matching the requested timestamp.
	// This should produce the same index unless the track had an inconsistent storage and frame rate.
	// E.g., times 0-1:00 had data, 1:00-1:30 had no data, and 1:30 to 2:00 had data again.
	int32 InitialFrameLocalOffset = 0;
	int32 StartFrameLocalOffset = 0;
	{
		const int32 GlobalRequestedStartFrame = RequestedStartFrame;
		const int32 GlobalRequestedInitialFrame = RequestedInitialFrame;
	
		RequestedStartFrame = InFrameData.FindFrameFromTimestamp(RequestedStartTimestamp, RequestedStartFrame);
		RequestedInitialFrame = InFrameData.FindFrameFromTimestamp(RequestedInitialTimestamp, RequestedInitialFrame);

		// Store any offset from the calculated localized frame to the actual localized frame.
		// This is needed to report the global buffered frames correctly. As a whole, the recording cares whether the
		// requested frames should be considered loaded, not the local frames of each track.
		StartFrameLocalOffset = GlobalRequestedStartFrame - RequestedStartFrame;
		InitialFrameLocalOffset = GlobalRequestedInitialFrame - RequestedInitialFrame;
	}
	
	int32 MaxFrames = InFrameData.MaxFrames;
	if (MaxFrames > 0)
	{
		if (RequestedFramesToLoad > 0)
		{
			// Don't go past requested frames or max frames. We add back in StartFrameLocalOffset because the start and initial
			// frames have been localized to the track, but the total frames we need may go to the end of the track, and we need to honor
			// the original request.
			MaxFrames = FMath::Min(MaxFrames, RequestedStartFrame + RequestedFramesToLoad + StartFrameLocalOffset + 1);
			// Ensure we're always at least streaming in the initial frame. This is usually the case, but depending on the track's
			// frames and how the RequestedFramesToLoad was divided, it may cut off the initial frame if it's the last frame of the track.
			MaxFrames = FMath::Max(MaxFrames, RequestedInitialFrame + 1);
		}

		const int32 LastFrame = MaxFrames - 1;
		check(LastFrame >= 0);
		
		IterationData.ForwardData.Reset();
		IterationData.ReverseData.Reset();

		const bool bIsNewIteration = IterationData.Status == FrameData::Private::FFrameBufferIterationData::New;
		
		// Load each frame from the initial frame, alternating right to left each frame. This creates a buffer to support
		// scrubbing each direction and makes sure the immediate frames are loaded first.

		int32 LastLoadedRightFrame, LastLoadedLeftFrame;
		
		if (bIsNewIteration)
		{
			LastLoadedRightFrame = RequestedInitialFrame;
			LastLoadedLeftFrame = RequestedInitialFrame;
			IterationData.Status = FrameData::Private::FFrameBufferIterationData::Active;
		}
		else
		{
			// We don't need to convert, as these are already localized.
			LastLoadedLeftFrame = IterationData.LastLoadedLeftFrame;
			LastLoadedRightFrame = IterationData.LastLoadedRightFrame + 1;
		}
		
		int32 RightFrameIdx = LastLoadedRightFrame;
		int32 LeftFrameIdx = LastLoadedLeftFrame - 1; // - 1 So we don't try to load the same initial frame when alternating to the left.

		bool bLoadRight = IterationData.bLoadRight; // Start right -> left
		
		// We could potentially optimize this further -- such as adjusting the ratio of ahead/behind frames to buffer based on whether
		// the recording is playing forward or reverse vs being scrubbed.

		// Clear loaded frames that aren't part of the required range.
		if (bIsNewIteration)
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameData::NewIteration"), STAT_ULiveLinkUAssetRecording_LoadFrameData_NewIteration, STATGROUP_LiveLinkHub);
			
			FScopeLock DataContainerLock(&DataContainerMutex);

			TRange<int32> RangeToLoad = RangeHelpers::Private::MakeInclusiveRange(RequestedStartFrame, LastFrame);
			TRange<int32> LoadedRange = InDataContainer.GetBufferedFrames();
			
			RangeHelpers::Private::TRangeArray<int32> FramesToUnload;
			{
				if (LoadedRange.GetLowerBoundValue() < RangeToLoad.GetLowerBoundValue())
				{
					TRange<int32> BeforeRange = RangeHelpers::Private::MakeInclusiveRange(LoadedRange.GetLowerBoundValue(), FMath::Min(RangeToLoad.GetLowerBoundValue(),LoadedRange.GetUpperBoundValue()));
					FramesToUnload.Add(BeforeRange);
				}

				if (LoadedRange.GetUpperBoundValue() > RangeToLoad.GetUpperBoundValue())
				{
					TRange<int32> AfterRange = RangeHelpers::Private::MakeInclusiveRange(FMath::Max(RangeToLoad.GetUpperBoundValue(), LoadedRange.GetLowerBoundValue()), LoadedRange.GetUpperBoundValue());
					FramesToUnload.Add(AfterRange);
				}
			}
			
			for (const TRange<int32>& Range : FramesToUnload)
			{
				MoveRangeToCache(Range, InDataContainer, InFrameData);
			}

			LoadedRange = InDataContainer.GetBufferedFrames();
			if (!LoadedRange.Contains(RequestedInitialFrame))
			{
				// In this case we aren't immediately loading frames, but will get to them eventually.
				TRange<int32> Intersection = TRange<int32>::Intersection(RangeToLoad, LoadedRange);
				MoveRangeToCache(Intersection, InDataContainer, InFrameData);
				
				LoadedRange = InDataContainer.GetBufferedFrames();
			}

			// If the range is partially loaded, default left/right load to loaded values to save iteration time.
			if (!LoadedRange.IsEmpty())
			{
				LastLoadedLeftFrame = LoadedRange.GetLowerBoundValue();
				LastLoadedRightFrame = LoadedRange.GetUpperBoundValue();
				LeftFrameIdx = LastLoadedLeftFrame;
				RightFrameIdx = LastLoadedRightFrame;
			}
		}
		
		// Verify we don't exceed the last frame, such as if this particular frame data ends before the recording's desired frame.
		{
			RightFrameIdx = FMath::Min(RightFrameIdx, LastFrame);
			LastLoadedRightFrame = FMath::Min(LastLoadedRightFrame, LastFrame);

			LeftFrameIdx = FMath::Min(LeftFrameIdx, LastFrame);
			LastLoadedLeftFrame = FMath::Min(LastLoadedLeftFrame, LastFrame);
		}

		if (LeftFrameIdx == RightFrameIdx)
		{
			// Don't try loading the same frame.
			RightFrameIdx++;
		}
		
		auto AlternateLoadDirection = [&bLoadRight](const bool bRightOnly)
		{
			bLoadRight = bRightOnly ? true : !bLoadRight;
		};

		const int32 BufferBatchSize = bForceSequential ? MaxFrames : GetDefault<ULiveLinkHubSettings>()->PlaybackBufferBatchSize;

		struct FBulkDataBatch
		{
			/** Reader containing unprocessed bulk data segment in memory. */
			TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> Reader;
			/** Minimum frame of the batch. */
			int32 MinFrame = 0;
			/** Maximum frame of the batch. */
			int32 MaxFrame = 0;
			/** Total frame count of the batch. Mainly to help if a batch is empty or not. */
			int32 Count = 0;
			/** If a frame number is included in this batch. */
			bool ContainsFrame(const int32 InFrame) const
			{
				return InFrame >= MinFrame && InFrame <= MaxFrame && Count != 0;
			}
		};

		// Determine ranges of disk data for left and right batches. The data must be contiguous, and will only
		// extend until a preloaded frame is found. These batches are so we can read multiple frames from bulk data at once,
		// saving on disk load time compared to loading the frames individually. Any data loaded here must still be processed
		// per-frame through a serializer.
		FBulkDataBatch RawFramesLeftBatch, RawFramesRightBatch;
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameData::CreateBatches"), STAT_ULiveLinkUAssetRecording_LoadFrameData_CreateBatches, STATGROUP_LiveLinkHub);

			// Initial allocation, splitting evenly between left and right.
			int32 HalfBatchSize = BufferBatchSize / 2;
			int32 LeftBatchSize = HalfBatchSize;
			int32 RightBatchSize = BufferBatchSize - HalfBatchSize; // Remainder goes to the right.

			const int32 AvailableLeftFrames = FMath::Max(LeftFrameIdx - RequestedStartFrame, 0);
			const int32 AvailableRightFrames = FMath::Max(MaxFrames - RightFrameIdx, 0);
		
			if (AvailableLeftFrames < LeftBatchSize)
			{
				RightBatchSize += (LeftBatchSize - AvailableLeftFrames); // Shift extra to the right.
				LeftBatchSize = AvailableLeftFrames;
			}

			if (AvailableRightFrames < RightBatchSize)
			{
				LeftBatchSize += (RightBatchSize - AvailableRightFrames); // Shift extra to the left.
				RightBatchSize = AvailableRightFrames;
			}

			// Make sure we're still within limits.
			LeftBatchSize = FMath::Clamp(LeftBatchSize, 0, InFrameData.bHasConsistentFrameSize ? AvailableLeftFrames : 1);
			RightBatchSize = FMath::Clamp(RightBatchSize, 0, InFrameData.bHasConsistentFrameSize ? AvailableRightFrames : 1);
			
			// Determine the min/max frames for each batch.
			if (LeftBatchSize > 0)
			{
				RawFramesLeftBatch.MinFrame = LeftFrameIdx - LeftBatchSize + 1;
				RawFramesLeftBatch.MaxFrame = LeftFrameIdx;

				if (RawFramesLeftBatch.MinFrame < 0)
				{
					RawFramesLeftBatch.MinFrame = 0;
				}

				// Shrink the batch to the first loaded frame, left to right since it's more likely frames for the left batch
				// will already be loaded the more to the right (center).
				for (int32 Idx = RawFramesLeftBatch.MinFrame; Idx < RawFramesLeftBatch.MaxFrame; ++Idx)
				{
					if (InDataContainer.IsFrameLoaded(Idx) || InFrameData.BufferedCache.ContainsFrame(Idx))
					{
						const int32 Remaining = (RawFramesLeftBatch.MaxFrame - Idx);
						LeftBatchSize -= Remaining;
						RawFramesLeftBatch.MaxFrame = Idx - 1;
						break;
					}
				}
				if (LeftBatchSize > 0)
				{
					RawFramesLeftBatch.Reader = LoadRawFramesFromDisk(RawFramesLeftBatch.MinFrame, LeftBatchSize, InFrameData);
				}
			}
			RawFramesLeftBatch.Count = FMath::Max(LeftBatchSize, 0);

			if (RightBatchSize > 0)
			{
				RawFramesRightBatch.MinFrame = RightFrameIdx;
				RawFramesRightBatch.MaxFrame = RightFrameIdx + RightBatchSize - 1;

				if (RawFramesRightBatch.MinFrame < 0)
				{
					RawFramesRightBatch.MinFrame = 0;
				}
				
				// Shrink the batch to the first loaded frame, right to left since it's more likely frames for the right batch
				// will already be loaded the more to the left (center).
				for (int32 Idx = RawFramesRightBatch.MaxFrame; Idx >= RawFramesRightBatch.MinFrame; --Idx)
				{
					if (InDataContainer.IsFrameLoaded(Idx) || InFrameData.BufferedCache.ContainsFrame(Idx))
					{
						const int32 Remaining = (Idx - RawFramesRightBatch.MinFrame) + 1;
						RightBatchSize -= Remaining;
						RawFramesRightBatch.MinFrame = Idx + 1;
						
						break;
					}
				}
				if (RightBatchSize > 0)
				{
					RawFramesRightBatch.Reader = LoadRawFramesFromDisk(RawFramesRightBatch.MinFrame, RightBatchSize, InFrameData);
				}
			}
			RawFramesRightBatch.Count = FMath::Max(RightBatchSize, 0);
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::LoadFrames)
		// Iterate through the entire range, loading frames from cache or disk. This runs until a batch cycle has completed
		// or the entire range is loaded.
		int32 FramesLoaded = 0;
		while (RightFrameIdx < MaxFrames || (!bForceSequential && LeftFrameIdx >= RequestedStartFrame))
		{
			if (bCancelStream)
			{
				break;
			}
			
			int32 FrameToLoad;
    
			if (bLoadRight)
			{
				if (RightFrameIdx >= MaxFrames)
				{
					bLoadRight = false;
					continue;
				}
				FrameToLoad = RightFrameIdx++;
				LastLoadedRightFrame = FrameToLoad;
			}
			else
			{
				if (LeftFrameIdx < RequestedStartFrame)
				{
					bLoadRight = true;
					continue;
				}
				FrameToLoad = LeftFrameIdx--;
				LastLoadedLeftFrame = FrameToLoad;
			}
			
			auto InsertFrame = [&](const TSharedPtr<FInstancedStruct>& InFrame, double InTimestamp)
			{
#if LIVELINKHUB_FRAME_DEBUG
				ensureAlways(!InFrameData.BufferIterationData.ForwardData.Timestamps.Contains(InTimestamp));
				ensureAlways(!InFrameData.BufferIterationData.ReverseData.Timestamps.Contains(InTimestamp));

				ensureAlways(!InDataContainer.Timestamps.Contains(InTimestamp));
				
				// Additional validation to ensure timestamps / frames are loaded in the correct order.
				for (int32 Idx = 1; Idx < InFrameData.BufferIterationData.ForwardData.Timestamps.Num(); ++Idx)
				{
					double LastTimestamp = InFrameData.BufferIterationData.ForwardData.Timestamps[Idx - 1];
					double CurrentTimestamp = InFrameData.BufferIterationData.ForwardData.Timestamps[Idx];
					ensureAlways(LastTimestamp < CurrentTimestamp);
				}
				for (int32 Idx = 1; Idx < InFrameData.BufferIterationData.ReverseData.Timestamps.Num(); ++Idx)
				{
					double LastTimestamp = InFrameData.BufferIterationData.ReverseData.Timestamps[Idx - 1];
					double CurrentTimestamp = InFrameData.BufferIterationData.ReverseData.Timestamps[Idx];
					ensureAlways(LastTimestamp < CurrentTimestamp);
				}
#endif
				if (bLoadRight)
				{
					InFrameData.BufferIterationData.ForwardData.Timestamps.Add(InTimestamp);
					InFrameData.BufferIterationData.ForwardData.RecordedData.Add(InFrame);
				}
				else
				{
					InFrameData.BufferIterationData.ReverseData.Timestamps.Insert(InTimestamp, 0);
					InFrameData.BufferIterationData.ReverseData.RecordedData.Insert(InFrame, 0);
				}
			};

			bool bIsFrameFullyLoaded = false;
			{
				DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameData::FindCachedFrame"), STAT_ULiveLinkUAssetRecording_LoadFrameData_FindCachedFrame, STATGROUP_LiveLinkHub);

				double ExistingTimestamp = 0.f;
				if (InDataContainer.IsFrameLoaded(FrameToLoad))
				{
					// Frame is in-memory and part of the active range.
					bIsFrameFullyLoaded = true;
				}
				else if (TSharedPtr<FInstancedStruct> ExistingFrame = InFrameData.BufferedCache.TryGetCachedFrame(FrameToLoad, ExistingTimestamp))
				{
					// Frame is still in memory, but was pending deletion. Move to the active range.
					InsertFrame(ExistingFrame, ExistingTimestamp);
					bIsFrameFullyLoaded = true;
				}
			}
			
			if (!bIsFrameFullyLoaded)
			{
				// Frame needs to be deserialized, and potentially loaded from disk.
				
				double Timestamp = 0.0;
				TSharedPtr<FInstancedStruct> DataInstancedStruct;
				TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> MemoryToUse;
				// Check if we have preloaded the raw data into memory.
				{
					int64 FramePosition = 0;
					if (RawFramesRightBatch.ContainsFrame(FrameToLoad))
					{
						MemoryToUse = RawFramesRightBatch.Reader;
						check(!RawFramesLeftBatch.ContainsFrame(FrameToLoad));
						FramePosition = InFrameData.GetRelativeFrameFilePosition(FrameToLoad - RawFramesRightBatch.MinFrame);
					}
					else if (RawFramesLeftBatch.ContainsFrame(FrameToLoad))
					{
						MemoryToUse = RawFramesLeftBatch.Reader;
						FramePosition = InFrameData.GetRelativeFrameFilePosition(FrameToLoad - RawFramesLeftBatch.MinFrame);
					}

					if (MemoryToUse.IsValid())
					{
						// The position should ideally be correct, but it's possible with various caching it is off.
						MemoryToUse->GetMemoryReader().Seek(FramePosition);
					}
				}
			
				if (!ensure(LoadFrameFromDisk(FrameToLoad, InFrameData, DataInstancedStruct, Timestamp, MemoryToUse)))
				{
					continue;
				}

				InsertFrame(DataInstancedStruct, Timestamp);
			}
			ensure(InDataContainer.Timestamps.Num() == InDataContainer.RecordedData.Num());
			
			FramesLoaded++;
			AlternateLoadDirection(bForceSequential);
			
			// Break the loop when enough frames have finished constituting a batch of frames.
			if (FramesLoaded > 0 && FramesLoaded % BufferBatchSize == 0)
			{
				break;
			}
		}

		IterationData.bLoadRight = bLoadRight;
		IterationData.LastLoadedLeftFrame = LastLoadedLeftFrame;
		IterationData.LastLoadedRightFrame = LastLoadedRightFrame;

		InDataContainer.LocalFrameOffset = InitialFrameLocalOffset;
		
		// Determine the iteration status, which will impact the overall async load loop.
		if (RightFrameIdx >= MaxFrames && LeftFrameIdx <= RequestedStartFrame)
		{
			IterationData.Status = FrameData::Private::FFrameBufferIterationData::Complete;
		}
		else if (bCancelStream || StreamingFrameChangedRequested())
		{
			IterationData.Status = FrameData::Private::FFrameBufferIterationData::Canceled;
		}
	}
}

bool ULiveLinkUAssetRecording::LoadFrameFromDisk(const int32 InFrame,
	const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData,
	TSharedPtr<FInstancedStruct>& OutFrame, double& OutTimestamp,
	const TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader>& InMemory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::LoadFrameFromDisk);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameFromDisk"), STAT_ULiveLinkUAssetRecording_LoadFrameFromDisk, STATGROUP_LiveLinkHub);
	OutTimestamp = 0.0;

	TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> Reader;
	
	// Either use preloaded memory if passed in, or load from bulk data directly.
	if (InMemory.IsValid())
	{
		Reader = InMemory;
	}
	else
	{
		const int64 FramePosition = InFrameData.GetFrameFilePosition(InFrame);
		AnimationData.SetBulkDataOffset(FramePosition);
		Reader = AnimationData.CreateBulkDataMemoryReader(InFrameData.GetFrameDiskSize(InFrame));
	}

	int32 ParsedFrameIdx = 0;
	Reader->GetMemoryReader() << ParsedFrameIdx;

	// Ensure the parsed frame index matches the expected frame
	if (!ensure(ParsedFrameIdx == InFrame))
	{
		UE_LOGF(LogLiveLinkHub, Error, "Frame index mismatch: expected %d, got %d", InFrame, ParsedFrameIdx);
		return false;
	}
			
	double Timestamp = 0.0;
	if (RecordingVersionBeingLoaded < UE::LiveLinkHub::Private::RecordingVersions::BulkTimestamps)
	{
		Reader->GetMemoryReader() << Timestamp;
	}
	else
	{
		check(InFrameData.Timestamps.IsValidIndex(InFrame));
		Timestamp = InFrameData.Timestamps[InFrame];
	}

	// Instantiate the animation frame.
	UScriptStruct* LoadedStruct = InFrameData.LoadedStruct.Get();
	if (LoadedStruct == nullptr)
	{
		UE_LOGF(LogLiveLinkHub, Error, "Script struct type not found.");
		return false;
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadFrameFromDisk::SerializeFrame"),
			STAT_ULiveLinkUAssetRecording_LoadFrameFromDisk_SerializeFrame, STATGROUP_LiveLinkHub);


		if (RecordingVersionBeingLoaded < UE::LiveLinkHub::Private::RecordingVersions::CompressionSupport)
		{
			FObjectAndNameAsStringProxyArchive StructAr(Reader->GetMemoryReader(), true);
			OutFrame = MakeShared<FInstancedStruct>(LoadedStruct);
			OutFrame->Serialize(StructAr);
		}
		else
		{
			int32 FrameSize = InFrameData.FrameDiskSizes[InFrame].Get<1>();
			int32 UncompressedSize = 0;

			Reader->GetMemoryReader() << UncompressedSize;

			const int32 HeaderSize = sizeof(FrameSize) + sizeof(UncompressedSize);

			// We've already read the header, so substract that amount from the frame size we're going to read next.
			FrameSize -= HeaderSize;
			check(FrameSize > 0);

			TArray<uint8> CompressedData;
			CompressedData.AddZeroed(FrameSize);

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::ReadCompressedData);
				Reader->GetMemoryReader().Serialize(CompressedData.GetData(), FrameSize);
			}

			TArray<uint8> UncompressedData = UE::LiveLinkHub::Compression::Decompress(InFrameData.CompressionMethodName, CompressedData, UncompressedSize);

			FMemoryReader UncompressedReader{ UncompressedData };

			FObjectAndNameAsStringProxyArchive StructAr(UncompressedReader, true);
			OutFrame = MakeShared<FInstancedStruct>(LoadedStruct);

			TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkUAssetRecording::DeserializeFrame);
			OutFrame->Serialize(StructAr);
		}
	}
	
	OutTimestamp = Timestamp;

	return true;
}

bool ULiveLinkUAssetRecording::LoadTimestampFromDisk(const int32 InFrame, const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData, double& OutTimestamp)
{
	check(RecordingVersionBeingLoaded < UE::LiveLinkHub::Private::RecordingVersions::BulkTimestamps);
	
	const int64 FramePosition = InFrameData.GetFrameFilePosition(InFrame);
	AnimationData.SetBulkDataOffset(FramePosition);
			
	const TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> Reader = AnimationData.CreateBulkDataMemoryReader(InFrameData.GetFrameDiskSize(InFrame));

	int32 ParsedFrameIdx = 0;
	Reader->GetMemoryReader() << ParsedFrameIdx;

	// Ensure the parsed frame index matches the expected frame
	if (ParsedFrameIdx != InFrame)
	{
		UE_LOGF(LogLiveLinkHub, Error, "Frame index mismatch: expected %d, got %d", InFrame, ParsedFrameIdx);
		return false;
	}
			
	Reader->GetMemoryReader() << OutTimestamp;

	return true;
}

TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> ULiveLinkUAssetRecording::LoadRawFramesFromDisk(const int32 InFrame,
	const int32 InNumFrames, const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::LoadRawFramesFromDisk"), STAT_ULiveLinkUAssetRecording_LoadRawFramesFromDisk, STATGROUP_LiveLinkHub);

	// Currently can only batch load multiple frames if frame size is consistent.
	check(InFrameData.bHasConsistentFrameSize || InNumFrames <= 1);
	
	// Seek to the beginning of the frames to load.
	const int64 FramePosition = InFrameData.GetFrameFilePosition(InFrame);
	AnimationData.SetBulkDataOffset(FramePosition);

	// Make sure the max frames don't exceed the maximum frames for this source.
	const int32 MaxFrames = FMath::Min(InNumFrames, InFrameData.MaxFrames - InFrame);
	check(MaxFrames >= 1);

	// Determine complete byte size to load.
	const int64 MaxByteSize = static_cast<int64>(MaxFrames) * static_cast<int64>(InFrameData.GetFrameDiskSize(InFrame));
	return AnimationData.CreateBulkDataMemoryReader(MaxByteSize);
}

void ULiveLinkUAssetRecording::EjectAndUnload()
{
	const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
	if (const TSharedPtr<FLiveLinkHubPlaybackController> Controller = LiveLinkHubModule.GetPlaybackController())
	{
		Controller->EjectAndUnload(nullptr, this);
	}
}

void ULiveLinkUAssetRecording::MoveFrameDataToContainer(FLiveLinkRecordingBaseDataContainer& InDataContainer,
	UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::MoveFrameDataToContainer"), STAT_ULiveLinkUAssetRecording_MoveFrameDataToContainer, STATGROUP_LiveLinkHub);

	// Don't mutex lock here, it should be locked from the calling method so multiple moves can be done under one lock.
	
#if LIVELINKHUB_FRAME_DEBUG
	// Additional validation to ensure timestamps / frames are loaded in the correct order.
	for (int32 Idx = 0; Idx < InFrameData.BufferIterationData.ForwardData.Timestamps.Num(); ++Idx)
	{
		ensureAlways(!InDataContainer.Timestamps.Contains(InFrameData.BufferIterationData.ForwardData.Timestamps[Idx]));
	}
#endif

	InDataContainer.RecordedDataStartFrame = InFrameData.BufferIterationData.LastLoadedLeftFrame;
	
	InDataContainer.Timestamps.Insert(MoveTemp(InFrameData.BufferIterationData.ReverseData.Timestamps), 0);
	InDataContainer.RecordedData.Insert(MoveTemp(InFrameData.BufferIterationData.ReverseData.RecordedData), 0);

	InDataContainer.Timestamps.Append(MoveTemp(InFrameData.BufferIterationData.ForwardData.Timestamps));
	InDataContainer.RecordedData.Append(MoveTemp(InFrameData.BufferIterationData.ForwardData.RecordedData));
			
#if LIVELINKHUB_FRAME_DEBUG
	// Additional validation to ensure timestamps / frames are loaded in the correct order.
	for (int32 Idx = 1; Idx < InDataContainer.Timestamps.Num(); ++Idx)
	{
		double LastTimestamp = InDataContainer.Timestamps[Idx - 1];
		double CurrentTimestamp = InDataContainer.Timestamps[Idx];
		ensureAlways(LastTimestamp < CurrentTimestamp);
	}
#endif
}

void ULiveLinkUAssetRecording::MoveRangeToCache(const TRange<int32>& InRange, FLiveLinkRecordingBaseDataContainer& InDataContainer,
	UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::MoveRangeToCache"), STAT_ULiveLinkUAssetRecording_MoveRangeToCache, STATGROUP_LiveLinkHub);
	if (!InRange.IsEmpty())
	{
		const int32 StartIndex = InRange.GetLowerBoundValue() - InDataContainer.RecordedDataStartFrame;
		const int32 CountToRemove = UE::LiveLinkHub::RangeHelpers::Private::GetRangeLength(InRange);

		if (StartIndex >= InDataContainer.Timestamps.Num()
			|| (StartIndex + CountToRemove > InDataContainer.Timestamps.Num())
			|| StartIndex < 0
			|| InDataContainer.Timestamps.IsEmpty())
		{
			return;
		}
					
		// Move to cache. This will be unloaded later.
		{
			FLiveLinkRecordingBaseDataContainer HistoryContainer;
			HistoryContainer.Timestamps.Reserve(CountToRemove);
			HistoryContainer.RecordedData.Reserve(CountToRemove);

			HistoryContainer.Timestamps.Append(&InDataContainer.Timestamps[StartIndex], CountToRemove);
			HistoryContainer.RecordedData.Append(&InDataContainer.RecordedData[StartIndex], CountToRemove);
			HistoryContainer.RecordedDataStartFrame = InDataContainer.RecordedDataStartFrame + StartIndex;
			HistoryContainer.LocalFrameOffset = InDataContainer.LocalFrameOffset;

			InFrameData.BufferedCache.FrameData.Add(MoveTemp(HistoryContainer));

			InFrameData.BufferedCache.TrimCache();
		}
					
		InDataContainer.Timestamps.RemoveAt(StartIndex, CountToRemove);
		InDataContainer.RecordedData.RemoveAt(StartIndex, CountToRemove);

		if (StartIndex == 0)
		{
			if (InDataContainer.Timestamps.Num() == 0)
			{
				InDataContainer.RecordedDataStartFrame = 0;
			}
			else
			{
				InDataContainer.RecordedDataStartFrame += CountToRemove;
			}
		}
	}
}

void ULiveLinkUAssetRecording::UpdateBufferedFrames()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::UpdateBufferedFrames"), STAT_ULiveLinkUAssetRecording_UpdateBufferedFrames, STATGROUP_LiveLinkHub);

	using namespace UE::LiveLinkHub;
	
	const FFrameRate GlobalFrameRate = GetGlobalFrameRate();
	
	FScopeLock Lock(&BufferedFrameMutex);

	// Find all buffered ranges and convert them to the global frame rate. We don't want to report the local frames that are buffered, but
	// the ranges as viewed by the recording/scrubber.
	BufferedFrameRanges.Reset();
	InactiveBufferedFrameRanges.Reset();

	const FQualifiedFrameTime LastFrameTime = GetLastFrameTime();
	const int32 LastFrameValue = LastFrameTime.Time.RoundToFrame().Value;
	
	for (TTuple<FLiveLinkSubjectKey, FrameData::Private::FFrameMetaData>& FrameDataKeyVal : FrameFileData)
	{
		FrameData::Private::FFrameMetaData& FrameData = FrameDataKeyVal.Value;
		RangeHelpers::Private::TRangeArray<int32> CacheBufferRanges = FrameData.BufferedCache.GetCacheBufferRanges(/*bIncludeOffset*/ true);
		for (TRange<int32>& Range : CacheBufferRanges)
		{
			Range = RangeHelpers::Private::ConvertRangeFrameRate(Range,FrameData.LocalFrameRate, GlobalFrameRate);
			if (Range.GetUpperBoundValue() > LastFrameValue)
			{
				Range.SetUpperBoundValue(LastFrameValue);
			}
		}
		
		TRange<int32> ActiveBufferFrames = RecordingData.FrameData.FindChecked(*FrameData.FrameDataSubjectKey.Get()).GetBufferedFrames(/*bIncludeOffset*/ true);
		ActiveBufferFrames = RangeHelpers::Private::ConvertRangeFrameRate(ActiveBufferFrames, FrameData.LocalFrameRate, GlobalFrameRate);

		if (ActiveBufferFrames.GetUpperBoundValue() > LastFrameValue)
		{
			ActiveBufferFrames.SetUpperBoundValue(LastFrameValue);
		}
		
		InactiveBufferedFrameRanges.Append(MoveTemp(CacheBufferRanges));
		BufferedFrameRanges.Add(MoveTemp(ActiveBufferFrames));
	}
}

void ULiveLinkUAssetRecording::WaitIfPaused_AsyncThread()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULiveLinkUAssetRecording::WaitIfPaused_AsyncThread"), STAT_ULiveLinkUAssetRecording_WaitIfPaused_AsyncThread, STATGROUP_LiveLinkHub);
	
	check(!IsInGameThread());
	
	if (bPauseStream)
	{
		OnStreamPausedEvent->Trigger();
		OnStreamUnpausedEvent->Wait();
	}
}

void ULiveLinkUAssetRecording::PauseStream()
{
	if (AsyncStreamTask.IsValid() && !AsyncStreamTask->IsDone())
	{
		OnStreamUnpausedEvent->Reset();
		bPauseStream = true;
		OnStreamPausedEvent->Wait();
	}
}

void ULiveLinkUAssetRecording::UnpauseStream()
{
	bPauseStream = false;
	OnStreamPausedEvent->Reset();
	OnStreamUnpausedEvent->Trigger();
}

void ULiveLinkUAssetRecording::OnPreGarbageCollect()
{
	PauseStream();
}

void ULiveLinkUAssetRecording::OnPostGarbageCollect()
{
	UnpauseStream();
}

void ULiveLinkUAssetRecording::FLiveLinkStreamAsyncTask::DoWork()
{
	int32 LastStartFrame = -1;
	int32 LastTotalFrames = -1;
	int32 LastInitialFrame = -1;
	while (LiveLinkRecording && !LiveLinkRecording->bCancelStream)
	{
		if (LastStartFrame != LiveLinkRecording->EarliestFrameToStream
			|| LastTotalFrames != LiveLinkRecording->TotalFramesToStream
			|| LastInitialFrame != LiveLinkRecording->InitialFrameToStream)
		{
			LastStartFrame = LiveLinkRecording->EarliestFrameToStream;
			LastTotalFrames = LiveLinkRecording->TotalFramesToStream;
			LastInitialFrame = LiveLinkRecording->InitialFrameToStream;
			LiveLinkRecording->LoadRecordingAsync(LiveLinkRecording->EarliestFrameToStream,
				LiveLinkRecording->InitialFrameToStream, LiveLinkRecording->TotalFramesToStream);

			if (LiveLinkRecording->bIsPerformingFullLoad)
			{
				// Execute on game thread since we're firing a delegate.
				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
					FSimpleDelegateGraphTask::FDelegate::CreateLambda([WeakRecording = MakeWeakObjectPtr(LiveLinkRecording)] ()
						{
							if (WeakRecording.IsValid())
							{
								WeakRecording->bIsFullyLoaded = true;
								WeakRecording->bIsPerformingFullLoad = false;
								WeakRecording->bIsUpgrading = false;
								WeakRecording->GetOnRecordingFullyLoaded().ExecuteIfBound();
							}
						}),
				TStatId(),
				nullptr,
				ENamedThreads::GameThread);
			}
		}
		else
		{
			LiveLinkRecording->WaitIfPaused_AsyncThread();
			FPlatformProcess::Sleep(0.002f);
		}
	}
}

#undef LIVELINKHUB_FRAME_DEBUG
