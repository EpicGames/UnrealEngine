// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Data/LiveLinkHubBulkData.h"
#include "LiveLinkFrameData.h"
#include "LiveLinkTypes.h"
#include "Recording/LiveLinkHubRecordingVersions.h"
#include "Recording/LiveLinkRecording.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"

#include "LiveLinkUAssetRecording.generated.h"

struct FLiveLinkPlaybackTracks;

/**
 * Asset containing all animation data stored as bulk data. This is loaded async in chunks dependent on the playhead position.
 * Overall recording length, framerate, and frame indices are based on the maximum track length and farthest timestamp.
 * When locating frames by indices the track will localize the frame index based on its internal framerate. All frame rates
 * are based strictly on the number of frames and the last timestamp of the track. True frame rate is up to the client.
 */
UCLASS()
class ULiveLinkUAssetRecording : public ULiveLinkRecording
{
public:
	GENERATED_BODY()

	virtual ~ULiveLinkUAssetRecording() override;
	
	virtual void Serialize(FArchive& Ar) override;

	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;

	virtual bool IsFullyLoaded() const override { return bIsFullyLoaded; }
	virtual bool IsSavingRecordingData() const override { return bIsSavingRecordingData; }
	virtual int32 GetMaxFrames() const override { return RecordingMaxFrames; }
	virtual double GetLastTimestamp() const override { return RecordingLastTimestamp; }
	virtual FQualifiedFrameTime GetLastFrameTime() const override;
	virtual FFrameRate GetGlobalFrameRate() const override;
	virtual void LoadEntireRecording() override;
	virtual bool IsLoadingEntireRecording() const override { return bIsPerformingFullLoad;}
	
	/** Save recording data to disk. */
	void SaveRecordingData();

	/** Load recording data from disk. */
	void LoadRecordingData(int32 InInitialFrame, int32 InNumFramesToLoad);

	/** Load the entire recording and notify this is for an upgrade. */
	void LoadEntireRecordingForUpgrade();
	
	/** Free memory and close file reader. */
	void UnloadRecordingData();

	/** Block until frames are loaded. */
	bool WaitForBufferedFrames(int32 InMinFrame, int32 InMaxFrame);

	/** The size in bytes of each animation frame. */
	int32 GetFrameDiskSize() const { return MaxFrameDiskSize; }
	
	/**
	 * Return all buffered frame ranges.
	 * @param bIncludeInactive If buffer ranges that are considered inactive should be included in the result.
	 */
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> GetBufferedFrameRanges(bool bIncludeInactive = true) const;
	
	/** Checks if a specific frame range is buffered. */
	bool IsFrameRangeBuffered(const TRange<int32>& InRange) const;

	/** If the recording is currently upgrading. */
	bool IsUpgrading() const { return bIsUpgrading; }
	
	/** Copy the asset's loaded recording data to a format suitable for playback in live link. */
	void CopyRecordingData(FLiveLinkPlaybackTracks& InOutLiveLinkPlaybackTracks) const;

	/** Initial setup of new recording data. */
	void InitializeNewRecordingData(FLiveLinkUAssetRecordingData&& InRecordingData, double InRecordingLengthSeconds);

	/** If the recording has performed its initial load. */
	bool HasPerformedInitialLoad() const { return bPerformedInitialLoad; }
	
	/** Calculate the framerate. */
	static FFrameRate CalculateFrameRate(const int32 InMaxFrames, const double InTime);

	DECLARE_DELEGATE(FOnRecordingInitialLoad);
	/** Delegate fired when the recording has performed an initial load. */
	FOnRecordingInitialLoad& GetOnRecordingInitialLoad() { return OnRecordingInitialLoadEvent; }
	
	DECLARE_DELEGATE(FOnRecordingFullyLoaded);
	/** Delegate fired when a recording has fully loaded. */
	FOnRecordingFullyLoaded& GetOnRecordingFullyLoaded() { return OnRecordingFullyLoadedEvent; }
	
private:
	/** Serialize the number of frames (array size) of the BaseDataContainer to the archive. */
	void SaveFrameData(FArchive* InFileWriter, const FLiveLinkSubjectKey& InSubjectKey, FLiveLinkRecordingBaseDataContainer& InBaseDataContainer);
	
	/** Initialize or update an async load. */
	void LoadRecordingAsync(int32 InStartFrame, int32 InCurrentFrame, int32 InNumFramesToLoad);

	/** Initial processing on a frame, finding the correct struct and offsets. The RecordingFileReader is assumed to be at the correct position. */
	bool LoadInitialFrameData(UE::LiveLinkHub::FrameData::Private::FFrameMetaData& OutFrameData);
	
	/**
	 * Load frame data to a data container. By default, it loads frames from the initial frame and then alternates in batches right then left.
	 *
	 * @param InFrameData The meta frame data being read and updated.
	 * @param InDataContainer The data container for loaded frames to be stored.
	 * @param RequestedStartFrame The farthest left start frame to load.
	 * @param RequestedInitialFrame The first frame to be loaded, such as the midpoint, or the start frame.
	 * @param RequestedFramesToLoad The total frames to try to load.
	 * @param bForceSequential Whether to force a sequential load, left to right, rather than alternating.
	 * In this case all frames from RequestedStartFrame to RequestedFramesToLoad are loaded.
	 */
	void LoadFrameData(UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData, FLiveLinkRecordingBaseDataContainer& InDataContainer,
		int32 RequestedStartFrame, int32 RequestedInitialFrame, int32 RequestedFramesToLoad, bool bForceSequential = false);

	/**
	 * Attempt to load a frame from bulk data.
	 * @param InFrame The frame number to process.
	 * @param InFrameData The frame data containing relevant information about this frame.
	 * @param OutFrame The deserialized frame.
	 * @param OutTimestamp The deserialized timestamp.
	 * @param InMemory [Optional] If we are reading from memory rather than bulk data directly.
	 */
	bool LoadFrameFromDisk(const int32 InFrame,
		const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData,
		TSharedPtr<FInstancedStruct>& OutFrame, double& OutTimestamp,
		const TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader>& InMemory = nullptr);

	/** Only load the timestamp from disk. */
	bool LoadTimestampFromDisk(const int32 InFrame, const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData, double& OutTimestamp);

	/** Load multiple frames as raw data from bulk data. They still need to be passed to LoadFrameFromDisk, so they can be deserialized. */
	TSharedPtr<FLiveLinkHubBulkData::FScopedBulkDataMemoryReader> LoadRawFramesFromDisk(const int32 InFrame,
		const int32 InNumFrames, const UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData);
	
	/** Eject this recording and make sure it is unloaded. */
	void EjectAndUnload();

	/** Move frame iteration data to the data container. */
	void MoveFrameDataToContainer(FLiveLinkRecordingBaseDataContainer& InDataContainer,
		UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData) const;

	/** Moves a range of frames from the container to the frame data cache. */
	void MoveRangeToCache(const TRange<int32>& InRange, FLiveLinkRecordingBaseDataContainer& InDataContainer,
		UE::LiveLinkHub::FrameData::Private::FFrameMetaData& InFrameData) const;
	
	/** Update the buffered frame range. */
	void UpdateBufferedFrames();
	
	/** Make the thread wait if we are paused. */
	void WaitIfPaused_AsyncThread();

	/** If the requested streaming frame has been changed. */
	bool StreamingFrameChangedRequested() const { return StreamingFrameChangeFromFrame != INDEX_NONE; }
	
	/** Signal and wait for the stream to be paused. */
	void PauseStream();

	/** Signal the stream can be resumed. */
	void UnpauseStream();
	
	/** Called before garbage collection. */
	void OnPreGarbageCollect();

	/** Called after garbage collection. */
	void OnPostGarbageCollect();
public:
	/** Recorded static and frame data. */
	UPROPERTY()
	FLiveLinkUAssetRecordingData RecordingData;
	
private:
	class FLiveLinkStreamAsyncTask : public FNonAbandonableTask
	{
	public:
		FLiveLinkStreamAsyncTask(ULiveLinkUAssetRecording* InLiveLinkRecording)
		{
			LiveLinkRecording = InLiveLinkRecording;
		}

		~FLiveLinkStreamAsyncTask()
		{
			if (LiveLinkRecording)
			{
				// Make sure we aren't waiting for a pause.
				LiveLinkRecording->OnStreamPausedEvent->Trigger();
			}
		}

		TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(LiveLinkStreamAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
		}

		void DoWork();

	private:
		TObjectPtr<ULiveLinkUAssetRecording> LiveLinkRecording;
	};

	/** The animation data -- bulk data stored within this uasset. */
	FLiveLinkHubBulkData AnimationData;
	
	/** The loaded frame data keys and position. Mapped by FLiveLinkSubjectKey to allow easy retrieval. */
	TMap<FLiveLinkSubjectKey, UE::LiveLinkHub::FrameData::Private::FFrameMetaData> FrameFileData;

	/** The maximum frames for this recording. */
	std::atomic<int32> RecordingMaxFrames = 0;

	/** The last time stamp of the recording.  */
	std::atomic<double> RecordingLastTimestamp = 0.0;

	/**
	 * The framerate of the track the last timestamp belongs to.
	 * We store this because we need to calculate all frames conversion between tracks to this
	 * "global" rate.
	 */
	FFrameRate RecordingLastTimestampFramerate;

	/** Frames buffered, divided into ranges. */
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> BufferedFrameRanges;
	/** Ranges that are buffered but considered inactive. */
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> InactiveBufferedFrameRanges;
	
	/** The first (left most) frame to stream. */
	int32 EarliestFrameToStream = 0;

	/** The initial frame to start streaming (the current playhead position).  */
	int32 InitialFrameToStream = 0;

	/** Total frames which should be streamed. */
	int32 TotalFramesToStream = 0;

	/** When the streaming frame has changed, signalling the current stream task should restart. */
	std::atomic<int32> StreamingFrameChangeFromFrame = INDEX_NONE;

	/** Signal that the stream should be canceled. */
	std::atomic<bool> bCancelStream = false;

	/** Signal that the stream should be paused. */
	std::atomic<bool> bPauseStream = false;

	/** True once a full initial load has been performed -- static + frame data. */
	std::atomic<bool> bPerformedInitialLoad = false;

	/** If we are currently saving recording frame data to disk. */
	std::atomic<bool> bIsSavingRecordingData = false;

	/** If we are currently loading all data from disk. */
	std::atomic<bool> bIsPerformingFullLoad = false;

	/** The maximum frame disk size across frame data. */
	std::atomic<int32> MaxFrameDiskSize = 0;

	/** Mutex for accessing the buffered frames. */
	mutable FCriticalSection BufferedFrameMutex;
	
	/** Mutex for accessing the data container from multiple threads. */
	mutable FCriticalSection DataContainerMutex;

	/** The thread streaming data from disk. */
	TUniquePtr<FAsyncTask<FLiveLinkStreamAsyncTask>> AsyncStreamTask;

	/** Handle for when gc is about to run. */
	FDelegateHandle OnPreGarbageCollectHandle;

	/** Handle for when gc has finished. */
	FDelegateHandle OnPostGarbageCollectHandle;

	/** Signalled when the stream is successfully paused. */
	FEventRef OnStreamPausedEvent = FEventRef(EEventMode::ManualReset);
	
	/** Signalled when the stream has been unpaused. */
	FEventRef OnStreamUnpausedEvent = FEventRef(EEventMode::ManualReset);

	/** If the recording is fully loaded into memory. */
	bool bIsFullyLoaded = false;

	/** Upgrade in progress. */
	bool bIsUpgrading = false;

	/** Called on the main thread when the recording has performed an initial load. */
	FOnRecordingFullyLoaded OnRecordingInitialLoadEvent;
	
	/** Called on the main thread when the recording has fully loaded. */
	FOnRecordingFullyLoaded OnRecordingFullyLoadedEvent;
	
	/** The current version of the recording. */
	const int32 CurrentRecordingVersion = UE::LiveLinkHub::Private::RecordingVersions::Latest;

	/** The version being currently loaded. */
	int32 RecordingVersionBeingLoaded = 0;
};
