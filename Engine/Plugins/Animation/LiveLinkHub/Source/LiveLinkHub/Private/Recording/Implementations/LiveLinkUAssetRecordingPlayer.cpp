// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkUAssetRecordingPlayer.h"

#include "HAL/IConsoleManager.h"
#include "LiveLinkHubLog.h"
#include "Recording/LiveLinkRecording.h"
#include "Settings/LiveLinkHubSettings.h"


class FLiveLinkPlaybackTrackIterator
{
public:
	virtual ~FLiveLinkPlaybackTrackIterator() = default;

	FLiveLinkPlaybackTrackIterator(FLiveLinkPlaybackTrack& InTrack, int32 InInitialIndex)
		: Track(InTrack)
		, FrameIndex(InInitialIndex)
	{
	}

	/** Advances to the next frame */
	void operator++()
	{
		Advance();
	}

	/* @Return True if there are more frames in this track. */
	explicit operator bool() const
	{
		return HasMoreFrames();
	}

	double FrameTimestamp() const
	{
		return FrameIndex >= 0 && FrameIndex < Track.Timestamps.Num() ? Track.Timestamps[FrameIndex] : 0.f;
	}

	const FInstancedStruct& FrameData() const
	{
		check(FrameIndex >= 0 && FrameIndex < Track.FrameData.Num());
		return *Track.FrameData[FrameIndex];
	}

	int32 CurrentIndex() const
	{
		return FrameIndex;
	}

protected:
	/* @Return True if there are more vertices on the component */
	virtual bool HasMoreFrames() const = 0;

	/* Advances to the next frame */
	virtual void Advance() = 0;

protected:
	/** Track that's currently being iterated. */
	FLiveLinkPlaybackTrack& Track;
	/** "Playhead" for this track */
	int32 FrameIndex = 0;
};

class FLiveLinkPlaybackTrackForwardIterator : public FLiveLinkPlaybackTrackIterator
{
public:
	FLiveLinkPlaybackTrackForwardIterator(FLiveLinkPlaybackTrack& InTrack, int32 InInitialIndex)
		: FLiveLinkPlaybackTrackIterator(InTrack, InInitialIndex)
	{
	}

private:
	/* @Return True if there are more vertices on the component */
	virtual bool HasMoreFrames() const override
	{
		return FrameIndex < Track.Timestamps.Num() && FrameIndex < Track.FrameData.Num();
	}

	/* Advances to the next frame */
	virtual void Advance() override
	{
		++FrameIndex;
	}
};

class FLiveLinkPlaybackTrackReverseIterator : public FLiveLinkPlaybackTrackIterator
{
public:
	FLiveLinkPlaybackTrackReverseIterator(FLiveLinkPlaybackTrack& InTrack, int32 InInitialIndex)
		: FLiveLinkPlaybackTrackIterator(InTrack, InInitialIndex)
	{
	}

private:
	/* @Return True if there are more vertices on the component */
	virtual bool HasMoreFrames() const override
	{
		return FrameIndex >= 0;
	}

	/* Advances to the next frame */
	virtual void Advance() override
	{
		--FrameIndex;
	}
};

void FLiveLinkPlaybackTrack::GetFramesUntil(const FQualifiedFrameTime& InFrameTime, TArray<FLiveLinkRecordedFrame>& OutFrames)
{
	const double InTimeSeconds = InFrameTime.AsSeconds();
	for (FLiveLinkPlaybackTrackForwardIterator It = FLiveLinkPlaybackTrackForwardIterator(*this, GetRelativeIndex(LastReadAbsoluteIndex)); It; ++It)
	{
		const double FrameTimestamp = It.FrameTimestamp();
		if (FrameTimestamp == LastTimeStamp)
		{
			// Generally the first iteration from LastReadAbsoluteIndex will trigger this,
			// but it's possible if the maximum buffered frames are small (ie 1), then
			// the LastReadAbsoluteIndex will now point to a different frame.
			continue;
		}
		
		if (FrameTimestamp > InTimeSeconds)
		{
			break;
		}

		LastReadRelativeIndex = It.CurrentIndex();
		LastReadAbsoluteIndex = LastReadRelativeIndex + StartIndexOffset;
		LastTimeStamp = FrameTimestamp;
		
		FLiveLinkRecordedFrame FrameToPlay;
		FrameToPlay.Data = It.FrameData();
		FrameToPlay.SubjectKey = SubjectKey;
		FrameToPlay.LiveLinkRole = LiveLinkRole;
		FrameToPlay.FrameIndex = LastReadAbsoluteIndex;

		OutFrames.Add(MoveTemp(FrameToPlay));
	}
}

void FLiveLinkPlaybackTrack::GetFramesUntilReverse(const FQualifiedFrameTime& InFrameTime, TArray<FLiveLinkRecordedFrame>& OutFrames)
{
	if (LastReadRelativeIndex == INDEX_NONE)
	{
		LastReadRelativeIndex = FrameData.Num();
		LastReadAbsoluteIndex = LastReadRelativeIndex + StartIndexOffset;
	}

	// We need to look up what the last frame would be if this was running forward, and then end on that frame.
	// Since we iterate in reverse, but all other operations like GoToFrame use forward look ahead, it's possible the time stamp comparison
	// will differ by a frame with a reverse look up. There's probably a better way of handling this.
	const int32 FinalFrameIndex = ConvertFrameTimeToFrameIndex(InFrameTime);

	const double InTimeSeconds = InFrameTime.AsSeconds();
	
	for (FLiveLinkPlaybackTrackReverseIterator It = FLiveLinkPlaybackTrackReverseIterator(*this, GetRelativeIndex(LastReadAbsoluteIndex)); It; ++It)
	{
		const double FrameTimestamp = It.FrameTimestamp();
		if (FrameTimestamp == LastTimeStamp)
		{
			// Generally the first iteration from LastReadAbsoluteIndex will trigger this,
			// but it's possible if the maximum buffered frames are small (ie 1), then
			// the LastReadAbsoluteIndex will now point to a different frame.
			continue;
		}
		
		if (FrameTimestamp < InTimeSeconds || FinalFrameIndex == LastReadRelativeIndex + StartIndexOffset)
		{
			break;
		}

		LastReadRelativeIndex = It.CurrentIndex();
		LastReadAbsoluteIndex = LastReadRelativeIndex + StartIndexOffset;
		LastTimeStamp = FrameTimestamp;
		
		FLiveLinkRecordedFrame FrameToPlay;
		FrameToPlay.Data = It.FrameData();
		FrameToPlay.SubjectKey = SubjectKey;
		FrameToPlay.LiveLinkRole = LiveLinkRole;
		FrameToPlay.FrameIndex = LastReadAbsoluteIndex;

		OutFrames.Add(MoveTemp(FrameToPlay));
	}
}

bool FLiveLinkPlaybackTrack::TryGetFrame(const FQualifiedFrameTime& InFrameTime, FLiveLinkRecordedFrame& OutFrame)
{
	const int32 RelativeIndex = ConvertFrameTimeToFrameIndex(InFrameTime);
	if (RelativeIndex >= 0 && RelativeIndex < FrameData.Num())
	{
		LastReadRelativeIndex = RelativeIndex;
		LastReadAbsoluteIndex = LastReadRelativeIndex + StartIndexOffset;
		
		FLiveLinkRecordedFrame FrameToPlay;
		FrameToPlay.Data = *FrameData[RelativeIndex];
		check(FrameToPlay.Data.IsValid());
		
		FrameToPlay.SubjectKey = SubjectKey;
		FrameToPlay.LiveLinkRole = LiveLinkRole;
		FrameToPlay.FrameIndex = LastReadAbsoluteIndex;

		OutFrame = MoveTemp(FrameToPlay);
		return true;
	}
	
	return false;
}

int32 FLiveLinkPlaybackTrack::ConvertFrameTimeToFrameIndex(const FQualifiedFrameTime& InFrameTime)
{
	const double InTimeSeconds = InFrameTime.AsSeconds();
	
	// Start by localizing the index based on the framerate. It's possible this is either exact, or closer to the desired position.
	int32 LocalizedIndex;
	if (LocalFrameRate.IsValid())
	{
		// Localize the frame time relative to this track's framerate.
		const FFrameTime LocalizedFrameTime = InFrameTime.ConvertTo(LocalFrameRate);
		LocalizedIndex = LocalizedFrameTime.RoundToFrame().Value;
	}
	else
	{
		LocalizedIndex = InFrameTime.Time.RoundToFrame().Value;
	}

	int32 CurrentIndex = GetRelativeIndex(LocalizedIndex);

	// Now find the closest timestamp without going over. The calculated CurrentIndex should be close to the desired location.
	if (Timestamps.IsValidIndex(CurrentIndex))
	{
		// Check if we're ahead of the desired time, iterate backwards if needed.
		while (CurrentIndex > 0 && Timestamps[CurrentIndex] > InTimeSeconds)
		{
			--CurrentIndex;
		}
	}
	else
	{
		// Fall back to full linear search.
		CurrentIndex = 0;
	}

	// If we're behind the desired time, iterate forward as needed.
	for (int32 Idx = CurrentIndex; Idx < Timestamps.Num(); ++Idx)
	{
		if (Timestamps[Idx] > InTimeSeconds)
		{
			break;
		}

		CurrentIndex = Idx;
	}

	return CurrentIndex;
}

TArray<FLiveLinkRecordedFrame> FLiveLinkPlaybackTracks::FetchNextFrames(const FQualifiedFrameTime& InFrameTime)
{
	TArray<FLiveLinkRecordedFrame> NextFrames;

	if (Tracks.Num())
	{
		// todo: sort frames by timestamp
		for (TTuple<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack>& TrackKeyVal : Tracks)
		{
			FLiveLinkPlaybackTrack& Track = TrackKeyVal.Value;
			Track.GetFramesUntil(InFrameTime, NextFrames);
		}
	}

	return NextFrames;
}

TArray<FLiveLinkRecordedFrame> FLiveLinkPlaybackTracks::FetchPreviousFrames(const FQualifiedFrameTime& InFrameTime)
{
	TArray<FLiveLinkRecordedFrame> PreviousFrames;

	if (Tracks.Num())
	{
		// todo: sort frames by timestamp
		for (TTuple<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack>& TrackKeyVal : Tracks)
		{
			FLiveLinkPlaybackTrack& Track = TrackKeyVal.Value;
			Track.GetFramesUntilReverse(InFrameTime, PreviousFrames);
		}
	}

	return PreviousFrames;
}

TArray<FLiveLinkRecordedFrame> FLiveLinkPlaybackTracks::FetchNextFramesAtIndex(const FQualifiedFrameTime& InFrameTime)
{
	TArray<FLiveLinkRecordedFrame> NextFrames;

	for (TTuple<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack>& TrackKeyVal : Tracks)
	{
		FLiveLinkPlaybackTrack& Track = TrackKeyVal.Value;
		FLiveLinkRecordedFrame Frame;
		if (Track.TryGetFrame(InFrameTime, Frame))
		{
			NextFrames.Add(MoveTemp(Frame));
		}
	}

	return NextFrames;
}

void FLiveLinkPlaybackTracks::Restart(int32 InIndex)
{
	for (TTuple<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack>& TrackKeyVal : Tracks)
	{
		FLiveLinkPlaybackTrack& Track = TrackKeyVal.Value;
		Track.Restart(InIndex);
	}
}

FFrameRate FLiveLinkPlaybackTracks::GetInitialFrameRate() const
{
	for (const TTuple<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack>& TrackKeyVal : Tracks)
	{
		const FLiveLinkPlaybackTrack& Track = TrackKeyVal.Value;
		if (Track.LiveLinkRole == nullptr && Track.FrameData.Num() > 0)
		{
			FLiveLinkFrameDataStruct FrameDataStruct;
			FrameDataStruct.InitializeWith(Track.FrameData[0]->GetScriptStruct(), (FLiveLinkBaseFrameData*)Track.FrameData[0]->GetMemory());

			return FrameDataStruct.GetBaseData()->MetaData.SceneTime.Rate;
		}
	}

	UE_LOGF(LogLiveLinkHub, Warning, "Could not find an initial framerate for the recording. Using the default value.");
	
	return FFrameRate(30, 1);
}

bool FLiveLinkUAssetRecordingPlayer::PreparePlayback(ULiveLinkRecording* CurrentRecording)
{
	// Ensure there nothing is playing and all settings are default. It's possible the CurrentRecording has settings that need to be cleared,
	// such as if this was just recorded and is now being loaded.
	ShutdownPlayback();

	ULiveLinkUAssetRecording* UAssetRecording = CastChecked<ULiveLinkUAssetRecording>(CurrentRecording);
	LoadedRecording = UAssetRecording;
	
	CurrentRecordingPlayback = FLiveLinkPlaybackTracks();

	return StreamPlayback(0);
}

void FLiveLinkUAssetRecordingPlayer::ShutdownPlayback()
{
	if (LoadedRecording.IsValid())
	{
		LoadedRecording->UnloadRecordingData();
	}
}

bool FLiveLinkUAssetRecordingPlayer::StreamPlayback(int32 InFromFrame)
{
	if (LoadedRecording.IsValid())
	{
		const int32 InitialFramesToBuffer = GetNumFramesToBuffer();
		LoadedRecording->LoadRecordingData(InFromFrame, InitialFramesToBuffer);

		// Make sure there are a few frames ready.
		if (LoadedRecording->WaitForBufferedFrames(InFromFrame, InFromFrame + 2))
		{
			// On initial load, the correct frame size may not be calculated prior until after waiting for the buffer,
			// update the correct number of frames and start buffering them.
			const int32 CurrentFramesToBuffer = GetNumFramesToBuffer();
			if (CurrentFramesToBuffer != InitialFramesToBuffer)
			{
				LoadedRecording->LoadRecordingData(InFromFrame, CurrentFramesToBuffer);
			}

			// Take the available recording data.
			LoadedRecording->CopyRecordingData(CurrentRecordingPlayback);
			return true;
		}
	}
	return false;
}

int32 FLiveLinkUAssetRecordingPlayer::GetNumFramesToBuffer() const
{
	int32 TotalFramesToBuffer = 0;
	if (LoadedRecording.IsValid())
	{
		const int32 FrameSize = LoadedRecording->GetFrameDiskSize();
		const int32 MaxFrameBufferSizeMB = GetDefault<ULiveLinkHubSettings>()->PlaybackFrameBufferSizeMB;
		const int32 MaxFrameBufferSizeBytes = MaxFrameBufferSizeMB * 1024 * 1024;

		// We divide total frames by 2, since they get doubled later to account for scrubbing in both directions.
		TotalFramesToBuffer = FrameSize > 0 ? MaxFrameBufferSizeBytes / FrameSize / 2 : 0;

		// Ensure at least a few frames can be buffered.
		constexpr int32 MinFrames = 3;
		TotalFramesToBuffer = FMath::Max(TotalFramesToBuffer, MinFrames);
	}

	return TotalFramesToBuffer;
}
