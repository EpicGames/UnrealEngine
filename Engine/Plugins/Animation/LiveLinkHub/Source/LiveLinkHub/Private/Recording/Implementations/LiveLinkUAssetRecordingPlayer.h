// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Recording/LiveLinkRecordingPlayer.h"

#include "Recording/Implementations/LiveLinkUAssetRecording.h"

/** Playback track that holds recorded data for a given subject. */
struct FLiveLinkPlaybackTrack
{
	/** Retrieve all frames from the last read index to the new frame time, forward-looking. */
	void GetFramesUntil(const FQualifiedFrameTime& InFrameTime, TArray<FLiveLinkRecordedFrame>& OutFrames);

	/** Retrieve all frames from the last read index to the new frame time, reverse-looking. */
	void GetFramesUntilReverse(const FQualifiedFrameTime& InFrameTime, TArray<FLiveLinkRecordedFrame>& OutFrames);
	
	/** Retrieve the frame at the read index. */
	bool TryGetFrame(const FQualifiedFrameTime& InFrameTime, FLiveLinkRecordedFrame& OutFrame);

	/**
	 * Given a frame time, find the closest index with a matching timestamp without going over.
	 * @param InFrameTime The frame time to convert to a frame index.
	 * @return The frame index or INDEX_NONE
	 */
	int32 ConvertFrameTimeToFrameIndex(const FQualifiedFrameTime& InFrameTime);

	/** Reset the LastReadIndex. */
	void Restart(int32 NewIndex = INDEX_NONE)
	{
		LastReadRelativeIndex = NewIndex < FrameData.Num() && NewIndex < Timestamps.Num() ? NewIndex : INDEX_NONE;
		LastReadAbsoluteIndex = LastReadRelativeIndex;
	}

	/** Convert an absolute frame index to a relative frame index. */
	int32 GetRelativeIndex(int32 InAbsoluteIndex) const
	{
		int32 RelativeIndex = InAbsoluteIndex - StartIndexOffset;
		RelativeIndex = FMath::Clamp(RelativeIndex, 0, FrameData.Num() - 1);
		return RelativeIndex;
	}

	/** Frame data to read. */
	TArray<TSharedPtr<FInstancedStruct>> FrameData;
	/** Timestamps for the frames in the track. */
	TConstArrayView<double> Timestamps;
	/** Used for static data. */
	TSubclassOf<ULiveLinkRole> LiveLinkRole;
	/** Subject key. */
	FLiveLinkSubjectKey SubjectKey;
	/** Index of the last relative frame that was read by the GetFrames method. */
	int32 LastReadRelativeIndex = -1;
	/** Index of the last absolute frame that was read by the GetFrames method. */
	int32 LastReadAbsoluteIndex = -1;
	/** The true index FrameData starts at. IE, if it starts at 5, then there are 5 prior frames [0..4] that aren't loaded. */
	int32 StartIndexOffset = 0;
	/** The frame rate of this track. Based only on the total frames and the final timestamp. */
	FFrameRate LocalFrameRate = FFrameRate(0, 0);

private:
	/** The last timestamp recorded. */
	double LastTimeStamp = -1.f;

	friend class FLiveLinkPlaybackTrackIterator;
};

/** Reorganized recording data to facilitate playback. */
struct FLiveLinkPlaybackTracks
{
	/** Get the next frames */
	TArray<FLiveLinkRecordedFrame> FetchNextFrames(const FQualifiedFrameTime& InFrameTime);

	/** Get the previous frames as if going in reverse */
	TArray<FLiveLinkRecordedFrame> FetchPreviousFrames(const FQualifiedFrameTime& InFrameTime);
	
	/** Get the next frame(s) at the index */
	TArray<FLiveLinkRecordedFrame> FetchNextFramesAtIndex(const FQualifiedFrameTime& InFrameTime);

	/** Restart all tracks. */
	void Restart(int32 InIndex);

	/** Retrieve the framerate of the first frame */
	FFrameRate GetInitialFrameRate() const;

public:
	/** LiveLink tracks to playback. */
	TMap<FLiveLinkSubjectKey, FLiveLinkPlaybackTrack> Tracks;
};

class FLiveLinkUAssetRecordingPlayer : public ILiveLinkRecordingPlayer
{
public:
	virtual bool PreparePlayback(class ULiveLinkRecording* CurrentRecording) override;

	virtual void ShutdownPlayback() override;
	
	virtual TArray<FLiveLinkRecordedFrame> FetchNextFramesAtTimestamp(const FQualifiedFrameTime& InFrameTime) override
	{
		if (StreamPlayback(InFrameTime.Time.RoundToFrame().Value))
		{
			return CurrentRecordingPlayback.FetchNextFrames(InFrameTime);
		}
		return TArray<FLiveLinkRecordedFrame>();
	}

	virtual TArray<FLiveLinkRecordedFrame> FetchPreviousFramesAtTimestamp(const FQualifiedFrameTime& InFrameTime) override
	{
		if (StreamPlayback(InFrameTime.Time.RoundToFrame().Value))
		{
			return CurrentRecordingPlayback.FetchPreviousFrames(InFrameTime);
		}
		return TArray<FLiveLinkRecordedFrame>();
	}

	virtual TArray<FLiveLinkRecordedFrame> FetchNextFramesAtIndex(const FQualifiedFrameTime& InFrameTime) override
	{
		if (StreamPlayback(InFrameTime.Time.RoundToFrame().Value))
		{
			return CurrentRecordingPlayback.FetchNextFramesAtIndex(InFrameTime);
		}
		return TArray<FLiveLinkRecordedFrame>();
	}

	virtual void RestartPlayback(int32 InIndex) override
	{
		CurrentRecordingPlayback.Restart(InIndex);
	}

	virtual FFrameRate GetInitialFramerate() override
	{
		return CurrentRecordingPlayback.GetInitialFrameRate();
	}

	virtual UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> GetBufferedFrameRanges() override
	{
		return LoadedRecording.IsValid() ? LoadedRecording->GetBufferedFrameRanges() : UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32>();
	}

private:
	/** Buffer playback around a given frame. */
	bool StreamPlayback(int32 InFromFrame);

	/** Retrieve the total frames to buffer, based on the size the user specified in the config file. */
	int32 GetNumFramesToBuffer() const;
	
private:
	/** All tracks for the current recording. */
	FLiveLinkPlaybackTracks CurrentRecordingPlayback;

	/** The recording currently loaded. */
	TWeakObjectPtr<ULiveLinkUAssetRecording> LoadedRecording;
};
