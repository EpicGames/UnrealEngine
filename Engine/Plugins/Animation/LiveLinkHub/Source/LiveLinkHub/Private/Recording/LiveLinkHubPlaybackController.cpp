// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubPlaybackController.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreGlobals.h"
#include "ILiveLinkClient.h"
#include "Implementations/LiveLinkUAssetRecordingPlayer.h"
#include "LiveLinkTimecodeProvider.h"
#include "LiveLinkHubCreatorAppMode.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubPlaybackSourceSettings.h"
#include "LiveLinkPreset.h"
#include "LiveLinkRecording.h"
#include "LiveLinkTypes.h"
#include "Features/IModularFeatures.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "LiveLinkClient.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubModule.h"
#include "Modules/ModuleManager.h"
#include "PackageTools.h"
#include "UI/Widgets/SLiveLinkHubPlaybackWidget.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubPlayback"

namespace UE::LiveLinkHub 
{
	static void ClearTimecodeProviderFrameHistory()
	{
		if (GEngine)
		{
			if (ULiveLinkTimecodeProvider* Provider = Cast< ULiveLinkTimecodeProvider>(GEngine->GetTimecodeProvider()))
			{
				Provider->ClearFrameHistory();
			}
		}
	}

	static void OverrideTimecodeEvaluationType()
	{
		UE::LiveLinkHubTimingUtils::Private::PlaybackEvaluationOverride = ELiveLinkTimecodeProviderEvaluationType::Latest;
		GetMutableDefault<ULiveLinkHubTimeAndSyncSettings>()->OnToggleTimecodeSettings();
	}

	static void ResetTimecodeEvaluationType()
	{
		UE::LiveLinkHubTimingUtils::Private::PlaybackEvaluationOverride.Reset();
		GetMutableDefault<ULiveLinkHubTimeAndSyncSettings>()->OnToggleTimecodeSettings();
	}
}

/**
 * Thread safe way to load and store a FQualifiedFrameTime. This is necessary because atomic<FQualifiedFrameTime> isn't
 * necessarily cross platform compatible when performing copy operations.
 */
class FLiveLinkHubAtomicQualifiedFrameTime {
public:
	FLiveLinkHubAtomicQualifiedFrameTime()
		: Value(FQualifiedFrameTime()) {}

	FLiveLinkHubAtomicQualifiedFrameTime(const FLiveLinkHubAtomicQualifiedFrameTime& Other) {
		FScopeLock Lock(&Other.Mutex);
		Value = Other.Value;
	}

	FLiveLinkHubAtomicQualifiedFrameTime& operator=(const FLiveLinkHubAtomicQualifiedFrameTime& Other) {
		if (this != &Other) {
			FScopeLock Lock(&Other.Mutex);
			Value = Other.Value;
		}
		return *this;
	}

	/** Set the underlying value. */
	void SetValue(const FQualifiedFrameTime& NewPlayhead) {
		FScopeLock Lock(&Mutex);
		Value = NewPlayhead;
	}

	/** Retrieve the underlying value. */
	FQualifiedFrameTime GetValue() const {
		FScopeLock Lock(&Mutex);
		return Value;
	}

private:
	/** Mutex for reading/writing underlying value. */
	mutable FCriticalSection Mutex;
	/** The underlying qualified frame time value. */
	FQualifiedFrameTime Value;
};

FLiveLinkHubPlaybackController::FLiveLinkHubPlaybackController()
{
	Client = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
	RecordingPlayer = MakeUnique<FLiveLinkUAssetRecordingPlayer>();
	Playhead = MakeShared<FLiveLinkHubAtomicQualifiedFrameTime, ESPMode::ThreadSafe>();
}

FLiveLinkHubPlaybackController::~FLiveLinkHubPlaybackController()
{
	bIsDestructing = true;
	bIsPlaying = false;
	if (!IsEngineExitRequested())
	{
		// Avoid doing a bunch of unnecessary work if the engine is currently exiting.
		Eject();
	}
	Stopping = true;
	PlaybackEvent->Trigger();

	if (Thread.IsValid())
	{
		Thread->WaitForCompletion();
		Thread.Reset();
	}
}

TSharedRef<SWidget> FLiveLinkHubPlaybackController::GetPlaybackWidget()
{
	if (TSharedPtr<SWidget> WidgetPtr = PlaybackWidget.Pin())
	{
		return WidgetPtr.ToSharedRef();
	}

	return SAssignNew(PlaybackWidget, SLiveLinkHubPlaybackWidget)
		.Visibility_Lambda([this]() { return IsInPlayback() || !bClosePlaybackTabOnEject ? EVisibility::Visible : EVisibility::Collapsed; })
		.IsPlaybackEnabled_Raw(this, &FLiveLinkHubPlaybackController::IsReady)
		.IsUpgrading_Lambda([this]()
		{
			if (const ULiveLinkUAssetRecording* Recording = Cast<ULiveLinkUAssetRecording>(GetRecording().Get()))
			{
				return Recording->IsUpgrading() && Recording->IsLoadingEntireRecording();
			}
			return false;
		})
		.OnGetUpgradePercent_Lambda([this]()
		{
			if (const ULiveLinkUAssetRecording* Recording = Cast<ULiveLinkUAssetRecording>(GetRecording().Get()))
			{
				using namespace UE::LiveLinkHub::RangeHelpers::Private;
				const TRangeArray<int32> BufferedFrames = GetBufferedFrameRanges();
				float Length = GetRangeLength(BufferedFrames);
				const float MaxLength = RecordingToPlay->GetMaxFrames();
				// Note: The range length is just an approximation. It reports the difference between the lowest frame and highest frame,
				// which could be on different tracks. It may be possible to report a frame range slightly higher than the actual max frames are
				// in the event the frame count of the upper end goes past the total frame length. That track's total frames should still be less
				// than our max frames. So we clamp as a quick fix to not report over 100%.
				Length = FMath::Clamp(Length, 0.f, MaxLength);
				// Wait for initial load otherwise the value can jump since we do a partial load, then unload first.
				if (MaxLength > 0.f && Recording->HasPerformedInitialLoad())
				{
					return (Length / MaxLength) * 100.f;
				}
			}
			return 0.f;
		})
		.OnPlayForward_Raw(this, &FLiveLinkHubPlaybackController::BeginPlayback, false)
		.OnPlayReverse_Raw(this, &FLiveLinkHubPlaybackController::BeginPlayback, true)
		.OnFirstFrame_Lambda([this]()
		{
			GoToTime(GetSelectionStartTime());
		})
		.OnLastFrame_Lambda([this]()
		{
			GoToTime(GetSelectionEndTime());
		})
		.OnPreviousFrame_Lambda([this]()
		{
			FQualifiedFrameTime CurrentTimeCopy = GetCurrentTime();
			CurrentTimeCopy.Time -= 1;

			// Clamp to selection start/end.
			if (CurrentTimeCopy.AsSeconds() < GetSelectionStartTime().AsSeconds())
			{
				CurrentTimeCopy = IsLooping() ? GetSelectionEndTime() : GetSelectionStartTime();
			}

			GoToTime(CurrentTimeCopy);
		})
		.OnNextFrame_Lambda([this]()
		{
			FQualifiedFrameTime CurrentTimeCopy = GetCurrentTime();
			CurrentTimeCopy.Time += 1;

			// Clamp to selection start/end.
			if (CurrentTimeCopy.AsSeconds() >= GetSelectionEndTime().AsSeconds())
			{
				CurrentTimeCopy = IsLooping() ? GetSelectionStartTime() : GetSelectionEndTime();
			}

			GoToTime(CurrentTimeCopy);
		})
		.OnSetPlayRate_Lambda([this](float InPlayRate)
		{
			SetPlayRate(InPlayRate);
		})
		.OnExitPlayback_Lambda([this]()
		{
			EjectAndUnload([]() {}, nullptr, bClosePlaybackTabOnEject);
		})
		.OnSelectRecording_Raw(this, &FLiveLinkHubPlaybackController::OnSelectRecording)
		.SetCurrentTime_Raw(this, &FLiveLinkHubPlaybackController::GoToTime)
		.GetViewRange_Lambda([this]()
		{
			return SliderViewRange;
		})
		.SetViewRange_Lambda([this](TRange<double> NewRange)
		{
			SliderViewRange = MoveTemp(NewRange);
		})
		.GetBufferRanges_Raw(this, &FLiveLinkHubPlaybackController::GetBufferedFrameRanges)
		.GetTotalLength_Raw(this, &FLiveLinkHubPlaybackController::GetLength)
		.GetCurrentTime_Raw(this, &FLiveLinkHubPlaybackController::GetCurrentTime)
		.GetSelectionStartTime_Raw(this, &FLiveLinkHubPlaybackController::GetSelectionStartTime)
		.SetSelectionStartTime_Raw(this,& FLiveLinkHubPlaybackController::SetSelectionStartTime)
		.GetSelectionEndTime_Raw(this, &FLiveLinkHubPlaybackController::GetSelectionEndTime)
		.SetSelectionEndTime_Raw(this,& FLiveLinkHubPlaybackController::SetSelectionEndTime)
		.IsPaused_Raw(this, &FLiveLinkHubPlaybackController::IsPaused)
		.IsInReverse_Raw(this, &FLiveLinkHubPlaybackController::IsPlayingInReverse)
		.IsLooping_Raw(this, &FLiveLinkHubPlaybackController::IsLooping)
		.OnSetLooping_Raw(this, &FLiveLinkHubPlaybackController::SetLooping)
		.GetFrameRate_Raw(this, &FLiveLinkHubPlaybackController::GetFrameRate)
		.OnScrubbingStart_Raw(this, &FLiveLinkHubPlaybackController::OnScrubbingStart)
		.OnScrubbingEnd_Raw(this, &FLiveLinkHubPlaybackController::OnScrubbingEnd);
}

void FLiveLinkHubPlaybackController::StartPlayback()
{
	ResumePlayback();
	PlaybackStartTime = FPlatformTime::Seconds();
	
	FPlatformMisc::MemoryBarrier();
	PlaybackEvent->Trigger();
}

void FLiveLinkHubPlaybackController::ResumePlayback()
{
	bIsPlaying = true;
	bIsPaused = false;

	UnpausePlaybackSubjects();

	FQualifiedFrameTime CurrentTime = GetCurrentTime();

	// Clamp to selection start/end.
	if (CurrentTime.AsSeconds() < GetSelectionStartTime().AsSeconds() && !bIsReverse)
	{
		CurrentTime = GetSelectionStartTime();
	}
	else if (CurrentTime.AsSeconds() > GetSelectionEndTime().AsSeconds() && bIsReverse)
	{
		CurrentTime = GetSelectionEndTime();
	}

	Playhead->SetValue(CurrentTime);
	
	StartTimestamp = CurrentTime.AsSeconds();
	// Force sync so interpolation doesn't interfere if the first frame isn't the current frame
	SyncToFrame(CurrentTime);
}

void FLiveLinkHubPlaybackController::PreparePlayback(ULiveLinkRecording* InLiveLinkRecording)
{
	if (RecordingPickerWindow)
	{
		// If we spawned a recording picker window, let's close it.
		RecordingPickerWindow->RequestDestroyWindow();

		// Set focus on playback widget so player can press spacebar to play right away.
		if (TSharedPtr<SWidget> WidgetPtr = PlaybackWidget.Pin())
		{
			FSlateApplication::Get().SetUserFocus(0, WidgetPtr);
		}
	}

	if (InLiveLinkRecording == nullptr)
	{
		UE_LOGF(LogLiveLinkHub, Warning, "Started a recording playback with an invalid recording.");
	}
	else if (InLiveLinkRecording != RecordingToPlay.Get())
	{
		TStrongObjectPtr<ULiveLinkRecording> RecordingStrongPtr(InLiveLinkRecording);

		auto PreparePlaybackCallback = [this, RecordingStrongPtr]()
		{
			if (RecordingStrongPtr.IsValid())
			{
				// Make sure PreparingPlayback is set to false when the scope exits.
				TGuardValue<bool> PreparingPlaybackGuard(bIsPreparingPlayback, true);

				FLiveLinkHub::Get()->GetTabManager()->TryInvokeTab(UE::LiveLinkHub::PlaybackTabId);

				RecordingToPlay.Reset(RecordingStrongPtr.Get());
				if (!RecordingPlayer->PreparePlayback(RecordingToPlay.Get()))
				{
					// Something failed during playback, cancel out.
					UE_LOGF(LogLiveLinkHub, Error, "Playback failed for '%ls', the file may be corrupted or unsupported.", *GetRecordingName());
					return;
				}

				const FQualifiedFrameTime RecordingLength = GetLength();
				if (RecordingLength.AsSeconds() <= 0.0)
				{
					// No point in continuing with an empty recording. Prevent bIsReady from being set, so the playback widget can't be used.
					UE_LOGF(LogLiveLinkHub, Warning, "Recording '%ls' is empty.", *GetRecordingName());
					return;
				}
				
				// The start and end of playback.
				SetSelectionStartTime(FQualifiedFrameTime(FFrameTime::FromDecimal(0.f), GetFrameRate()));
				SetSelectionEndTime(RecordingLength);
		
				// The range the user sees.
				SliderViewRange = TRange<double>(SelectionStartTime.AsSeconds(), SelectionEndTime.AsSeconds());
		
				const FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), ULiveLinkPreset::StaticClass(), TEXT("RecordingRollbackPreset"));
				RollbackPreset.Reset(NewObject<ULiveLinkPreset>(GetTransientPackage(), UniqueName));
				// Save the current state of the sources/subjects in a rollback preset.
				RollbackPreset->BuildFromClient();

				// This clears out any live streams which might be occurring. They will be restored when exiting playback later.
				{
					FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);

					LiveLinkClient.RemoveAllSources();
					LiveLinkClient.Tick();
				}

				RecordingToPlay->RecordingPreset->ApplyToClientLatent([this](bool)
				{
					UE::LiveLinkHub::OverrideTimecodeEvaluationType();

					bIsReady = true;
					SyncToFrame(FQualifiedFrameTime()); // Needed to establish connection with client
					
					// We should already be on the game thread, but we want to execute OnPlaybackReady delegate on the next tick
					// since we're still operating within a callback of ApplyToClient. If a caller performs an action
					// that prompts another ApplyToClientCall while inside one, we can hit an exception.
					FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
						FSimpleDelegateGraphTask::FDelegate::CreateLambda([this]()
						{
							OnPlaybackReady().Broadcast();
						}),
						TStatId(),
						nullptr,
						ENamedThreads::GameThread);
				});
			}
		};

		if (RecordingToPlay.IsValid())
		{
			constexpr bool bClosePlaybackTab = false;
			EjectAndUnload(PreparePlaybackCallback, nullptr, bClosePlaybackTab);
		}
		else
		{
			PreparePlaybackCallback();
		}
	}
	else
	{
		// Already loaded and ready.
		OnPlaybackReady().Broadcast();
	}
}

void FLiveLinkHubPlaybackController::PlayRecording(ULiveLinkRecording* InLiveLinkRecording)
{
	PreparePlayback(InLiveLinkRecording);
}

void FLiveLinkHubPlaybackController::BeginPlayback(bool bInReverse)
{
	const bool bReverseChange = bIsReverse != bInReverse;
	bIsReverse = bInReverse;
	
	// Either we are paused and should unpause, or we are toggling forward/reverse play modes.
	if (bIsPaused || !bIsPlaying || bReverseChange)
	{
		if (ShouldRestart())
		{
			// Check if we're at the end of the recording and restart, ie user pressed play again.
			RestartPlayback();
		}
		else
		{
			if (bIsReverse)
			{
				RecordingPlayer->RestartPlayback(GetCurrentFrame().Value);
			}
			
			// Resume as normal for anywhere else in the recording.
			PlaybackStartTime = FPlatformTime::Seconds();
		}
		
		ResumePlayback();

		// Note: We don't want to clear frame history when pausing because timecode would become invalid.
		UE::LiveLinkHub::ClearTimecodeProviderFrameHistory();
	}
	else if (bIsPlaying)
	{
		PausePlayback();
	}

	FPlatformMisc::MemoryBarrier();
	PlaybackEvent->Trigger();
}

void FLiveLinkHubPlaybackController::RestartPlayback()
{
	if (bIsStopping)
	{
		// A stop request is in progress. It's possible a restart request also is occurring because of looping.
		return;
	}
	const bool bOldReverse = bIsReverse; // Stop playback resets reverse
	StopPlayback();
	StartTimestamp = GetCurrentTime().AsSeconds();
	RecordingPlayer->RestartPlayback(GetCurrentFrame().Value);
	bIsPlaying = true;
	bIsReverse = bOldReverse;

	UE::LiveLinkHub::ClearTimecodeProviderFrameHistory();
}

void FLiveLinkHubPlaybackController::PausePlayback()
{
	bIsPaused = true;
	PausePlaybackSubjects();
}

void FLiveLinkHubPlaybackController::StopPlayback()
{
	bIsStopping = true;
	bIsPlaying = false;
	
	// Wait for the playback thread to exit...
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	if (CurrentThreadId != Thread->GetThreadID() && IsInGameThread() /* Only wait if request came from gamethread, not the playback thread */)
	{
		while(!bIsPlaybackWaiting) { }
	}
	
	const bool bReverse = bIsReverse.load();
	Playhead->SetValue(bReverse ? GetSelectionEndTime() : GetSelectionStartTime());

	PlaybackStartTime = FPlatformTime::Seconds();

	RecordingPlayer->RestartPlayback();
	bIsReverse = false;
	bIsStopping = false;
}

void FLiveLinkHubPlaybackController::Eject(TFunction<void()> CompletionCallback)
{
	bIsReady = false;
	
	StopPlayback();

	LastStaticFrameIndex.Empty();
	
	bIsPaused = false;
	RecordingPlayer->RestartPlayback(0);

	SetSelectionStartTime(FQualifiedFrameTime(FFrameTime::FromDecimal(0), GetFrameRate()));
	SetSelectionEndTime(FQualifiedFrameTime(FFrameTime::FromDecimal(0), GetFrameRate()));
	Playhead->SetValue(FQualifiedFrameTime(FFrameTime::FromDecimal(0), GetFrameRate()));
	StartTimestamp = 0.f;

	// Recording is done, clear the pointer.
	RecordingPlayer->ShutdownPlayback();

	if (RecordingToPlay.IsValid())
	{
		// It's possible the initial latent action is still in progress if the user ejected this recording immediately after playing.
		RecordingToPlay->RecordingPreset->CancelLatentAction();
	}
	RecordingToPlay.Reset();
	
	if (RollbackPreset.IsValid())
	{
		RollbackPreset->ApplyToClientLatent([CompletionCallback](bool)
		{
			if (CompletionCallback)
			{
				CompletionCallback();
			}
		});
	}
	else if (CompletionCallback)
	{
		CompletionCallback();
	}
}

void FLiveLinkHubPlaybackController::EjectAndUnload(const TFunction<void()>& EjectCompletionCallback,
	const ULiveLinkRecording* InRecording, bool bClosePlaybackTab)
{
	const ULiveLinkRecording* Recording = InRecording ? InRecording : RecordingToPlay.Get();
	UPackage* Package = Recording ? Recording->GetPackage() : nullptr;
	const bool bIsSavingRecordingData = Recording && Recording->IsSavingRecordingData();

	if (bClosePlaybackTab)
	{
		if (TSharedPtr<SDockTab> PlaybackTab = FLiveLinkHub::Get()->GetTabManager()->FindExistingLiveTab(UE::LiveLinkHub::PlaybackTabId))
		{
			PlaybackTab->RequestCloseTab();
		}
	}

	// Only eject if this is in reference to the current recording playing.
	if (InRecording == nullptr || InRecording == RecordingToPlay.Get())
	{
		Eject(EjectCompletionCallback);
	}
	
	// Unload on the next tick since this could have been called from multistep operations, such as rename or delete. We need to completely
	// unload so when loading in the future the bulk data archive will be attached correctly.
	if (Package && !bIsSavingRecordingData)
	{
		constexpr bool bUnloadNextTick = true;
		UnloadRecordingPackage(Package, bUnloadNextTick);
	}

	UE::LiveLinkHub::ResetTimecodeEvaluationType();
}

void FLiveLinkHubPlaybackController::UnloadRecordingPackage(const TWeakObjectPtr<UPackage>& InPackage, bool bUnloadNextTick)
{
	if (!InPackage.IsValid() || PackagesUnloading.Contains(InPackage))
	{
		return;
	}
	
	auto UnloadPackage = [PackageToUnload = InPackage, this](double = 0.f, float = 0.f) -> EActiveTimerReturnType
	{
		ensure(PackagesUnloading.Remove(PackageToUnload) > 0);
			
		if (PackageToUnload.IsValid() && !PackageToUnload->IsDirty() && !PackageToUnload->HasAnyPackageFlags(PKG_IsSaving))
		{
			UPackageTools::FUnloadPackageParams UnloadParams({ PackageToUnload.Get() });
			const bool bUnloaded = UPackageTools::UnloadPackages(UnloadParams);
			ensure(bUnloaded);
		}
		
		return EActiveTimerReturnType::Stop;
	};

	PackagesUnloading.Add(InPackage);

	if (bUnloadNextTick)
	{
		const TSharedPtr<FLiveLinkHub> LiveLinkHub = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkHub();
		LiveLinkHub->GetRootWindow()->RegisterActiveTimer(
			0.f,
			FWidgetActiveTimerDelegate::CreateLambda(UnloadPackage));
	}
	else
	{
		UnloadPackage();
	}
}

void FLiveLinkHubPlaybackController::OnSelectRecording()
{
	SAssignNew(RecordingPickerWindow, SWindow)
		.Title(LOCTEXT("PickRecordingTitle", "Pick a recording."))
		.ClientSize(FVector2D(750, 440))
		.SizingRule(ESizingRule::UserSized)
		[
			FLiveLinkHub::Get()->GetRecordingListController()->MakeRecordingList()
		];

	if (GEditor)
	{
		GEditor->EditorAddModalWindow(RecordingPickerWindow.ToSharedRef());
	}
}

void FLiveLinkHubPlaybackController::GoToTime(FQualifiedFrameTime InTime)
{
	UE::LiveLinkHub::ClearTimecodeProviderFrameHistory();

	// Stop needs to occur to restart playback.
	StopPlayback();

	for (const FLiveLinkSubjectKey& SubjectKey : GetPlaybackSubjects())
	{
		Client->ClearSubjectsFrames_AnyThread(SubjectKey);
	}

	const double TimeDouble = InTime.AsSeconds();
	
	PlaybackStartTime -= TimeDouble;
	Playhead->SetValue(InTime);

	SyncToFrame(InTime);
}

FQualifiedFrameTime FLiveLinkHubPlaybackController::GetSelectionStartTime() const
{
	return SelectionStartTime;
}

void FLiveLinkHubPlaybackController::SetSelectionStartTime(FQualifiedFrameTime InTime)
{
	SelectionStartTime = InTime;
}

FQualifiedFrameTime FLiveLinkHubPlaybackController::GetSelectionEndTime() const
{
	return SelectionEndTime;
}

void FLiveLinkHubPlaybackController::SetSelectionEndTime(FQualifiedFrameTime InTime)
{
	SelectionEndTime = InTime;
}

FQualifiedFrameTime FLiveLinkHubPlaybackController::GetLength() const
{
	if (RecordingToPlay)
	{
		return RecordingToPlay->GetLastFrameTime();
	}

	return FQualifiedFrameTime();
}

FQualifiedFrameTime FLiveLinkHubPlaybackController::GetCurrentTime() const
{
	const FQualifiedFrameTime Time = Playhead->GetValue();
	return Time;
}

FFrameNumber FLiveLinkHubPlaybackController::GetCurrentFrame() const
{
	return GetCurrentTime().Time.GetFrame();
}

FFrameRate FLiveLinkHubPlaybackController::GetFrameRate() const
{
	if (RecordingToPlay.IsValid())
	{
		return RecordingToPlay->GetGlobalFrameRate();
	}

	return FFrameRate(60, 1);
}

UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> FLiveLinkHubPlaybackController::GetBufferedFrameRanges() const
{
	return RecordingPlayer->GetBufferedFrameRanges();
}

FString FLiveLinkHubPlaybackController::GetRecordingName() const
{
	return RecordingToPlay.IsValid() ? RecordingToPlay.Get()->GetName() : "Unknown";
}

void FLiveLinkHubPlaybackController::Start()
{
	FString ThreadName = TEXT("LiveLinkHub Playback Controller ");
	ThreadName.AppendInt(FAsyncThreadIndex::GetNext());

	Thread.Reset(FRunnableThread::Create(this, *ThreadName, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask()));
}

void FLiveLinkHubPlaybackController::Stop()
{
	Stopping = true;
}

uint32 FLiveLinkHubPlaybackController::Run()
{
	while (!Stopping.load())
	{
		bIsPlaybackWaiting = true;
		PlaybackEvent->Wait();
		bIsPlaybackWaiting = false;

		while (bIsPlaying)
		{
			if (IsEngineExitRequested())
			{
				Stopping = true;
				bIsDestructing = true;
				break;
			}

			if (bIsPaused)
			{
				FPlatformProcess::Sleep(0.002f);
			}
			else
			{
				const bool bSynced = SyncToPlayhead();

				auto SetPlayhead = [&]()
				{
					const double TimeDilatedDelta = (FPlatformTime::Seconds() - PlaybackStartTime) * PlayRate;
					double Position = bIsReverse ? StartTimestamp - TimeDilatedDelta : StartTimestamp + TimeDilatedDelta;
					Position = FMath::Clamp(Position, GetSelectionStartTime().AsSeconds(), GetSelectionEndTime().AsSeconds());
					const FFrameRate FrameRate = GetFrameRate();
					Playhead->SetValue(FQualifiedFrameTime(FrameRate.AsFrameTime(Position), FrameRate));
				};
				SetPlayhead();
			
				// Don't sleep if we pushed frames since that can take a small amount of time.
				if (!bSynced)
				{
					FPlatformProcess::Sleep(0.002f);
					SetPlayhead();
				}
			}
			
			if (ShouldRestart())
			{
				if (bLoopPlayback && RecordingToPlay->LengthInSeconds != 0 && !bIsPaused)
				{
					RestartPlayback();
				}
				else
				{
					// Stop playback
					break;
				}
			}

		}

		// If the loop ended because the recording is over.
		bIsPlaying = false;
		
		if (!bIsDestructing)
		{
			// Trigger the playback finished delegate on the game thread.
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FLiveLinkHubPlaybackController::OnPlaybackFinished_Internal), TStatId(), nullptr, ENamedThreads::GameThread);
		}
	}

	return 0;
}

void FLiveLinkHubPlaybackController::OnPlaybackFinished_Internal()
{
	if (!bIsDestructing && GIsRunning) // Can crash otherwise, such as if we are closing the app.
	{
		PlaybackFinishedDelegate.Broadcast();
	}
}

void FLiveLinkHubPlaybackController::PushSubjectData(const FLiveLinkRecordedFrame& NextFrame, bool bForceSync)
{
	// If we're sending static data
	if (NextFrame.LiveLinkRole)
	{
		// Make sure we only push static data if it has changed.
    	if (const int32* Idx = LastStaticFrameIndex.Find(NextFrame.SubjectKey))
    	{
    		if (*Idx == NextFrame.FrameIndex)
    		{
    			return;
    		}
    	}

    	LastStaticFrameIndex.Add(NextFrame.SubjectKey, NextFrame.FrameIndex);
	
		FLiveLinkStaticDataStruct StaticDataStruct;
		StaticDataStruct.InitializeWith(NextFrame.Data.GetScriptStruct(), (FLiveLinkBaseStaticData*)NextFrame.Data.GetMemory());
		Client->PushSubjectStaticData_AnyThread(NextFrame.SubjectKey, NextFrame.LiveLinkRole, MoveTemp(StaticDataStruct));
	}
	else
	{
		FLiveLinkFrameDataStruct FrameDataStruct;
		FrameDataStruct.InitializeWith(NextFrame.Data.GetScriptStruct(), (FLiveLinkBaseFrameData*)NextFrame.Data.GetMemory());
		
		if (bForceSync)
		{
			FrameDataStruct.GetBaseData()->MetaData.StringMetaData.Add(TEXT("ForceSync"), TEXT("true"));
		}
		Client->PushSubjectFrameData_AnyThread(NextFrame.SubjectKey, MoveTemp(FrameDataStruct));
	}
}

bool FLiveLinkHubPlaybackController::SyncToPlayhead()
{
	const FQualifiedFrameTime FrameTime = Playhead->GetValue();
	TArray<FLiveLinkRecordedFrame> NextFrames = bIsReverse ? RecordingPlayer->FetchPreviousFramesAtTimestamp(FrameTime)
		: RecordingPlayer->FetchNextFramesAtTimestamp(FrameTime);
	
	for (const FLiveLinkRecordedFrame& NextFrame : NextFrames)
	{
		// todo: Have to forcesync -- interpolation fails due to improper frame times
		const bool bForceSync = bIsReverse;
		PushSubjectData(NextFrame, bForceSync);
	}
	
	return NextFrames.Num() > 0;
}

bool FLiveLinkHubPlaybackController::SyncToFrame(const FQualifiedFrameTime& InFrameTime)
{
	TArray<FLiveLinkRecordedFrame> NextFrames = RecordingPlayer->FetchNextFramesAtIndex(InFrameTime);
	if (NextFrames.Num() == 0)
	{
		UE_LOGF(LogLiveLinkHub, Warning, "No frame loaded for frame number %d", InFrameTime.Time.GetFrame().Value);
	}
	for (const FLiveLinkRecordedFrame& NextFrame : NextFrames)
	{
		PushSubjectData(NextFrame, true);
	}

	return NextFrames.Num() > 0;
}

bool FLiveLinkHubPlaybackController::ShouldRestart() const
{
	const FFrameNumber CurrentFrame = GetCurrentFrame();
	return RecordingToPlay.IsValid() && ((bIsReverse && CurrentFrame <= GetSelectionStartTime().Time.GetFrame())
		|| (!bIsReverse && CurrentFrame >= GetSelectionEndTime().Time.GetFrame()));
}

void FLiveLinkHubPlaybackController::OnResumePlayback()
{
	if (ensureAlways(IsInPlayback() && IsPaused()))
	{
		constexpr bool bInReverse = false;
		BeginPlayback(bInReverse);
	}
}

void FLiveLinkHubPlaybackController::OnPausePlayback()
{
	if (ensureAlways(IsInPlayback() && !IsPaused()))
	{
		PausePlayback();
	}
}

void FLiveLinkHubPlaybackController::OnScrubbingStart()
{
	bStartedScrubbingPaused = IsPaused();

	if (!IsPaused())
	{
		PausePlayback();
	}
}

void FLiveLinkHubPlaybackController::OnScrubbingEnd()
{
	if (!bStartedScrubbingPaused)
	{
		BeginPlayback(false);
	}
}

TArray<FLiveLinkSubjectKey> FLiveLinkHubPlaybackController::GetPlaybackSubjects() const
{
	TArray<FLiveLinkSubjectKey> Subjects = Client->GetSubjects(false, false);

	return Subjects.FilterByPredicate([this] (const FLiveLinkSubjectKey& SubjectKey)
	{
		const UObject* SourceSettings = Client->GetSourceSettings(SubjectKey.Source);
		return SourceSettings && SourceSettings->IsA<ULiveLinkHubPlaybackSourceSettings>();
	});
}

void FLiveLinkHubPlaybackController::PausePlaybackSubjects() const
{
	for (const FLiveLinkSubjectKey& Subject : GetPlaybackSubjects())
	{
		Client->PauseSubject_AnyThread(Subject.SubjectName);
	}
}

void FLiveLinkHubPlaybackController::UnpausePlaybackSubjects() const
{
	for (const FLiveLinkSubjectKey& Subject : GetPlaybackSubjects())
	{
		Client->UnpauseSubject_AnyThread(Subject.SubjectName);
	}
}

#undef LOCTEXT_NAMESPACE