// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"

#include "HAL/Event.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "Recording/LiveLinkRecordingPlayer.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

#include <atomic>

#include "LiveLinkRecording.h"
#include "LiveLinkRecordingRangeHelpers.h"

enum class ELiveLinkHubPlaybackState : uint8;
class ILiveLinkClient;
struct FLiveLinkRecordingData;
class FRunnableThread;
class ULiveLinkPreset;
class SWidget;
class FLiveLinkHubAtomicQualifiedFrameTime;

class FLiveLinkHubPlaybackController : public FRunnable
{
public:
	FLiveLinkHubPlaybackController();
	virtual ~FLiveLinkHubPlaybackController() override;

	/** Get the playback widget and create it if it doesn't exists. */
	TSharedRef<SWidget> GetPlaybackWidget();

	/** Apply the recording's preset then prepare the data needed to playback. */
	void PreparePlayback(ULiveLinkRecording* InLiveLinkRecording);
	
	/** Start playing a livelink recording. */
	void PlayRecording(ULiveLinkRecording* InLiveLinkRecording);

	/** The current recording. */
	const TStrongObjectPtr<ULiveLinkRecording>& GetRecording() const { return RecordingToPlay; }
	
	/** Start playing the currently prepared recording. */
	void BeginPlayback(bool bInReverse);

	/** Prepare to restart the playback. */
	void RestartPlayback();

	/** Pause playback. */
	void PausePlayback();
	
	/** Stop playing a livelink recording. */
	void StopPlayback();

	/** Stop playback and restore the previous settings. */
	void Eject(TFunction<void()> CompletionCallback = nullptr);

	/**
	 * Eject this recording and make sure it is unloaded.
	 * @param EjectCompletionCallback Callback after ejection occurs.
	 * @param InRecording The specific recording to eject and unload. If null, the RecordingToPlay is used. 
	 */
	void EjectAndUnload(const TFunction<void()>& EjectCompletionCallback = nullptr,
		const ULiveLinkRecording* InRecording = nullptr, bool bClosePlaybackTab = true);

	/**
	 * Completely unload a recording package.
	 *
	 * Optionally allow the package to be unloaded on the next tick. This is to help with ensuring the package is fully unloaded, such as if
	 * the unload was requested during a multistep operation, like renaming the asset. When the asset is saved, the bulk data file archive is
	 * lost, and a reload is needed to reattach it.
	 * 
	 * @param InPackage The package of the recording.
	 * @param bUnloadNextTick Whether the package should unload on the next tick or now.
	 */
	void UnloadRecordingPackage(const TWeakObjectPtr<UPackage>& InPackage, bool bUnloadNextTick = false);
	
	/** Create a pop up window that shows available recordings to play. */
	void OnSelectRecording();

	/** Go to a specific time. */
	void GoToTime(FQualifiedFrameTime InTime);
	
	/** Retrieve the selection start time. */
	FQualifiedFrameTime GetSelectionStartTime() const;

	/** Set the selection start time. */
	void SetSelectionStartTime(FQualifiedFrameTime InTime);

	/** Retrieve the selection end time. */
	FQualifiedFrameTime GetSelectionEndTime() const;

	/** Set the selection end time. */
	void SetSelectionEndTime(FQualifiedFrameTime InTime);

	/** Retrieve the length of the recording. */
	FQualifiedFrameTime GetLength() const;
	
	/** Retrieve the playhead. */
	FQualifiedFrameTime GetCurrentTime() const;

	/** Retrieve the current frame of the animation. */
	FFrameNumber GetCurrentFrame() const;

	/** Retrieve the current framerate. */
	FFrameRate GetFrameRate() const;

	/** Retrieve buffered frame range. */
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> GetBufferedFrameRanges() const;
	
	/** If the controller is ready for commands. */
	bool IsReady() const
	{
		return bIsReady && GetRecording() && !GetRecording()->IsLoadingEntireRecording();
	}
	
	/** If a recording is loaded into the controller. */
	bool IsInPlayback() const
	{
		return GetRecording().IsValid();
	}

	/** Returns whether we've started or are actively playing a recording. */ 
	bool IsPlaying() const
	{
		return bIsPlaying.load();
	}

	/** If playback is paused. */
	bool IsPaused() const
	{
		return bIsPaused.load() || !IsPlaying();
	}

	/** If playback is playing in reverse. */
	bool IsPlayingInReverse() const
	{
		return bIsReverse.load();
	}

	/** Returns whether the recording is set to loop. */
	bool IsLooping() const
	{
		return bLoopPlayback.load();
	}

	/** Set whether a recording should loop. */
	void SetLooping(bool bInLoop)
	{
		bLoopPlayback.store(bInLoop);
	}

	/** Sets the play rate (time dilation multiplier) of playback. */
	void SetPlayRate(float InPlayRate)
	{
		PlayRate = InPlayRate;
		// Reset the PlaybackStartTime and playhead since we're modifying the play rate.
		GoToTime(GetCurrentTime());
		if (!bIsPaused)
		{
			BeginPlayback(bIsReverse);
		}
	}

	/** Retrieve the current recording's name. */
	FString GetRecordingName() const;
	
	/** Delegate called when playback is finished (if recording is not set to loop). */
	FTSSimpleMulticastDelegate& OnPlaybackFinished()
	{
		return PlaybackFinishedDelegate;
	}

	/** Delegate called when recording playback is ready. */
	FTSSimpleMulticastDelegate& OnPlaybackReady()
	{
		return PlaybackReadyDelegate;
	}

	/** Set whether the playback tab should be closed when the eject button is pressed. */
	void SetClosePlaybackTabOnEject(bool bCloseOnEject)
	{
		bClosePlaybackTabOnEject = bCloseOnEject;
	}
	
	/** Create the playback thread. */
	void Start();
	
	//~ Begin FRunnable Interface
	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override { }
	//~ End FRunnable Interface

private:
	/** Trigger the playback thread to start reading data. */
	void StartPlayback();
	/** Resume on the playback thread. */
	void ResumePlayback();

	/** Handler called when playback is finished on the playback thread. Is responsible for resetting the livelink state to what it was before we started playback. */
	void OnPlaybackFinished_Internal();
	
	/**
	 * Send data to the client.
	 * @param NextFrame The frame to push.
	 * @param bForceSync Force the client to sync directly to this frame, discarding all other frames.
	 */
	void PushSubjectData(const FLiveLinkRecordedFrame& NextFrame, bool bForceSync = false);

	/**
	 * Sync the animation to the current playhead value.
	 *
	 * @return true if any frames were pushed.
	 */
	bool SyncToPlayhead();

	/** Force sync to a specific frame. */
	bool SyncToFrame(const FQualifiedFrameTime& InFrameTime);

	/** Checks if the current playback settings indicates the recording should restart. */
	bool ShouldRestart() const;

	/** Handles the user clicking the play button to resume playback. */
	void OnResumePlayback();

	/** Handles the user clicking the pause button to resume playback. */
	void OnPausePlayback();

	/** Handles pausing the recording while we're scrubbing. */
	void OnScrubbingStart();

	/** Handles resuming the recording when we end scrubbing. */
	void OnScrubbingEnd();

	/** Get all playback subjects. */
	TArray<FLiveLinkSubjectKey> GetPlaybackSubjects() const;

	/** Pauses all playback subjects. */
	void PausePlaybackSubjects() const;

	/** Unpauses all playback subjects. */
	void UnpausePlaybackSubjects() const;

private:
	/** Cached pointer to the playback widget. */
	TWeakPtr<SWidget> PlaybackWidget;
	/** If the system has established a connection with the client. */
	bool bIsReady = false;
	/** Flag for terminating the thread loop */
	std::atomic<bool> Stopping = false;
	/** Thread to do playback on **/
	TUniquePtr<FRunnableThread> Thread;
	/** Event signaling that a recording is available for playback. */
	FEventRef PlaybackEvent = FEventRef();
	/** If the playback thread is waiting. */
	std::atomic<bool> bIsPlaybackWaiting = false;
	/** Whether a recording is playing. */
	std::atomic<bool> bIsPlaying = false;
	/** Stop requested. */
	std::atomic<bool> bIsStopping = false;
	/** Whether we're currently paused. */
	std::atomic<bool> bIsPaused = false;
	/** If the recording is playing in reverse. */
	std::atomic<bool> bIsReverse = false;
	/** If the controller is destructing. */
	std::atomic<bool> bIsDestructing = false;
	/** The timestamp of the animation when first playing. Can be > 0 when running in reverse. */
	std::atomic<double> StartTimestamp = 0.f;
	/** Indicates that we're in the process of preparing the playback. Used by the OnSourceRemoved callback to make sure we don't eject during the PreparePlayback step. */
	bool bIsPreparingPlayback = false;
	/** LiveLinkRecording to play.  */
	TStrongObjectPtr<ULiveLinkRecording> RecordingToPlay;
	/** Delegate called when a recording playback is finished (if it's not looping). */
	FTSSimpleMulticastDelegate PlaybackFinishedDelegate;
	/** Delegate called when recording playback is ready. */
	FTSSimpleMulticastDelegate PlaybackReadyDelegate;
	/** Preset used to rollback the hub to its previous state after playing a recording. */
	TStrongObjectPtr<ULiveLinkPreset> RollbackPreset;
	/** Atomic bool keeping track of whether we should loop the playback. */
	std::atomic<bool> bLoopPlayback = true;
	/** Implementation of the playback functionality. */
	TUniquePtr<ILiveLinkRecordingPlayer> RecordingPlayer;

	/** LiveLinkClient used to transmit the data to connected clients. */
	ILiveLinkClient* Client = nullptr;
	/** Time that the playback started. */
	double PlaybackStartTime = 0.0;
	double TimeDilationAdjustedCurrentTime = 0.0;
	std::atomic<float> PlayRate = 1;

	/** Playhead for the current playback. */
	TSharedPtr<FLiveLinkHubAtomicQualifiedFrameTime, ESPMode::ThreadSafe> Playhead;

	/** The view range of the slider, defaults to start/end time. */
	TRange<double> SliderViewRange = TRange<double>(0.f, 0.f);

	/** The playback selection start time. */
	FQualifiedFrameTime SelectionStartTime;

	/** The playback selection end time. */
	FQualifiedFrameTime SelectionEndTime;

	/** Packages in the process of unloading. */
	TSet<TWeakObjectPtr<UPackage>> PackagesUnloading;

	/** The last sent static frame index. Used to ensure we only send static data when necessary. */
	TMap<FLiveLinkSubjectKey, int32> LastStaticFrameIndex;

	/** Window that displays the list of recordings that can be played back. */
	TSharedPtr<class SWindow> RecordingPickerWindow;

	/** Keeps track of whether the recording was paused when starting to scrub to ensure we don't pause if we scrub while playing. */
	bool bStartedScrubbingPaused = false;

	/** Whether to close the playback tab when ejecting a recording. */
	bool bClosePlaybackTabOnEject = false;
};
