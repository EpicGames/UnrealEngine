// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"

#include "LiveLinkPlaybackSource.h"
#include "LiveLinkPresetTypes.h"

#include "LiveLinkHubPlaybackSourceFactory.generated.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubPlaybackSourceFactory"

/** Factory used to create playback sources when we're playing back a livelink recording. */
UCLASS()
class ULiveLinkHubPlaybackSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	//~ Begin ULiveLinkSourceFactory interface
	virtual FText GetSourceDisplayName() const override
	{
		return LOCTEXT("PlaybackSourceDisplayName", "Playback Source");
	}

	virtual FText GetSourceTooltip() const
	{
		return LOCTEXT("PlaybackSourceTooltip", "Mock Live Link source used to playback Live Link data that was recorded.");
	}

	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const
	{
		return MakeShared<FLiveLinkPlaybackSource>(ConnectionString);
	}

	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FLiveLinkSourcePreset& SourcePreset)
	{
		return MakeShared<FLiveLinkPlaybackSource>(*SourcePreset.SourceType.ToString());
	}
	//~ End ULiveLinkSourceFactory interface
};

#undef LOCTEXT_NAMESPACE
