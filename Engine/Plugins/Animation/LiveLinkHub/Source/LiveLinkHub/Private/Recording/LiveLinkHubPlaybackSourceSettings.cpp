// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubPlaybackSourceSettings.h"

#include "LiveLinkHubPlaybackSourceFactory.h"
#include "LiveLinkSourceFactory.h"
#include "LiveLinkSubjectSettings.h"


ULiveLinkHubPlaybackSourceSettings::ULiveLinkHubPlaybackSourceSettings()
{
	Factory = ULiveLinkHubPlaybackSourceFactory::StaticClass();
}

FText ULiveLinkHubPlaybackSourceSettings::GetSourceNameOverride(ULiveLinkSubjectSettings* SubjectSettings, FText SourceType)
{
	check(SubjectSettings);

	if (!SubjectSettings->OriginalSourceName.IsNone())
	{
		return FText::FromName(SubjectSettings->OriginalSourceName);
	}

	return SourceType;
}
