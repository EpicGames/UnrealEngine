// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"

#include "LiveLinkHubPlaybackSourceSettings.generated.h"

/** Settings class for playback sources, which provides the factory class for them. */
UCLASS()
class ULiveLinkHubPlaybackSourceSettings : public ULiveLinkSourceSettings
{
public:
	GENERATED_BODY()

	ULiveLinkHubPlaybackSourceSettings();

	virtual FText GetSourceNameOverride(class ULiveLinkSubjectSettings* SubjectSettings, FText SourceType) override;
};
