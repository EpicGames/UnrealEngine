// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubRecordingController.h"

#include "Implementations/LiveLinkUAssetRecorder.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "LiveLinkRecorder.h"
#include "SLiveLinkHubRecordingView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.RecordingController"

FLiveLinkHubRecordingController::FLiveLinkHubRecordingController()
{
	RecorderImplementation = MakeShared<FLiveLinkUAssetRecorder>();
}

TSharedRef<SWidget> FLiveLinkHubRecordingController::MakeRecordToolbarEntry()
{
	return SNew(SLiveLinkHubRecordingView)
		.CanRecord_Raw(this, &FLiveLinkHubRecordingController::CanRecord)
		.IsRecording_Raw(this, &FLiveLinkHubRecordingController::IsRecording)
		.OnStartRecording_Raw(this, &FLiveLinkHubRecordingController::StartRecording)
		.OnStopRecording_Raw(this, &FLiveLinkHubRecordingController::StopRecording)
		.ToolTipText_Raw(this, &FLiveLinkHubRecordingController::GetRecordingButtonTooltip);
}

void FLiveLinkHubRecordingController::StartRecording()
{
	ILiveLinkRecordingSession::Get().OnRecordingStarted().Broadcast();

	RecorderImplementation->StartRecording();
}
	
void FLiveLinkHubRecordingController::StopRecording()
{
	ILiveLinkRecordingSession::Get().OnRecordingStopped().Broadcast();

	RecorderImplementation->StopRecording();

	if (GetDefault<ULiveLinkHubSettings>()->bIncrementTakeOnRecordingEnd)
	{
		ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();
		SessionInfo.SetTakeNumber(SessionInfo.GetTakeNumber() + 1);
	}
}

bool FLiveLinkHubRecordingController::IsRecording() const
{
	return RecorderImplementation->IsRecording();
}

bool FLiveLinkHubRecordingController::CanRecord() const
{
	return RecorderImplementation->CanRecord(&ErrorMessage);
}

FText FLiveLinkHubRecordingController::GetRecordingButtonTooltip() const
{
	if (ErrorMessage.IsEmpty())
	{
		return IsRecording() ?
			LOCTEXT("RecordingStopTooltip", "Stop a recording") :
			LOCTEXT("RecordingStartTooltip", "Start a recording");
	}
	
	return ErrorMessage;
}

void FLiveLinkHubRecordingController::RecordStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData)
{
	RecorderImplementation->RecordStaticData(SubjectKey, Role, StaticData);
}
	
void FLiveLinkHubRecordingController::RecordFrameData(const FLiveLinkSubjectKey& SubjectKey, const FLiveLinkFrameDataStruct& FrameData)
{
	RecorderImplementation->RecordFrameData(SubjectKey, FrameData);
}

void FLiveLinkHubRecordingController::RecordFromExistingAndSave(ULiveLinkRecording* InRecording, const FString& InFilePath, bool bUpgradeInPlace)
{
	RecorderImplementation->SaveFromExistingRecording(InRecording, InFilePath, bUpgradeInPlace);
}

bool FLiveLinkHubRecordingController::OpenSaveAsPrompt(FString& OutName, const FString& InInitialName) const
{
	constexpr bool bOpenSaveAs = true;
	return RecorderImplementation->GetSavePresetPackageName(OutName, InInitialName, bOpenSaveAs);
}

#undef LOCTEXT_NAMESPACE
