// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "LiveLinkRecorder.h"
#include "LiveLinkTypes.h"
#include "Templates/PimplPtr.h"

class ILiveLinkRecorder;
class ULiveLinkUAssetRecording;
struct FInstancedStruct;
struct FLiveLinkRecordingBaseDataContainer;
class SWidget;

class FLiveLinkHubRecordingController
{
public:
	FLiveLinkHubRecordingController();

	/** Create the toolbar entry for starting/stopping recordings. */
	TSharedRef<SWidget> MakeRecordToolbarEntry();

	/** Start recording livelink data. */
	void StartRecording();
	
	/** Stop recording livelink data and prompt the user for a save location. */
	void StopRecording();

	/** Returns whether we're currently recording. */
	bool IsRecording() const;

	/** Returns whether we can begin recording. */
	bool CanRecord() const;

	/** The tool tip to display on the recording button. */
	FText GetRecordingButtonTooltip() const;

	/** Record static data in the current recording. */
	void RecordStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<class ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData);

	/** Record frame data in the current recording. */
	void RecordFrameData(const FLiveLinkSubjectKey& SubjectKey, const FLiveLinkFrameDataStruct& FrameData);

	/** Record from an existing recording. */
	void RecordFromExistingAndSave(ULiveLinkRecording* InRecording, const FString& InFilePath, bool bUpgradeInPlace = false);

	/** Open a save as window and retrieve a file path. */
	bool OpenSaveAsPrompt(FString& OutName, const FString& InInitialName) const;
private:
	/** Recorder used to serialize livelink data into a given format. */
	TSharedPtr<ILiveLinkRecorder> RecorderImplementation;
	
	/** Any error text associated with the recording, for display on the recording button tooltip. */
	mutable FText ErrorMessage;
};
