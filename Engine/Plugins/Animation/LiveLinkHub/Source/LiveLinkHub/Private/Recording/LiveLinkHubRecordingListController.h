// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "IContentBrowserSingleton.h"
#include "DirectoryWatcherModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDirectoryWatcher.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubPlaybackController.h"
#include "LiveLinkHubRecordingController.h"
#include "SLiveLinkHubRecordingListView.h"
#include "LiveLinkRecording.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.RecordingList"

class FLiveLinkHubRecordingListController
{
public:
	FLiveLinkHubRecordingListController(const TSharedRef<FLiveLinkHub>& InLiveLinkHub)
		: LiveLinkHub(InLiveLinkHub)
	{
		TStringBuilder<128> MountPackageName;
		TStringBuilder<128> MountFilePath;
		TStringBuilder<128> RelPath;

		if (!FPackageName::TryGetMountPointForPath(TEXT("/Game"), MountPackageName, MountFilePath, RelPath))
		{
			return;
		}

		ContentDirectory = MountFilePath.ToString();

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		AssetRegistry.ScanPathsSynchronous({ ContentDirectory }, /*bForceRescan=*/ true);

		// Add a directory watcher callback to refresh recordings.
		if (IDirectoryWatcher* Watcher = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher")).Get())
		{
			Watcher->RegisterDirectoryChangedCallback_Handle(ContentDirectory, IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FLiveLinkHubRecordingListController::OnContentDirectoryChanged), DirectoryChangedHandle, IDirectoryWatcher::IncludeDirectoryChanges);
		}
	}

	~FLiveLinkHubRecordingListController()
	{
		FDirectoryWatcherModule* DirectoryWatcher = FModuleManager::GetModulePtr<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));

		if (DirectoryWatcher && DirectoryWatcher->Get())
		{
			DirectoryWatcher->Get()->UnregisterDirectoryChangedCallback_Handle(ContentDirectory, DirectoryChangedHandle);
		}

		if (GEditor && GEditor->IsTimerManagerValid())
		{
			GEditor->GetTimerManager()->ClearTimer(DirectoryScanTimerHandle);
		}
	}

	/** Create the list's widget. */
	TSharedRef<SWidget> MakeRecordingList()
	{
		return SNew(SLiveLinkHubRecordingListView)
			.OnImportRecording_Raw(this, &FLiveLinkHubRecordingListController::OnImportRecording);
	}

private:
	/** Handler called when a recording is clicked which will start the recording. */
	void OnImportRecording(const FAssetData& AssetData)
	{
		if (const TSharedPtr<FLiveLinkHub> HubPtr = LiveLinkHub.Pin())
		{
			if (HubPtr->GetRecordingController()->IsRecording())
			{
				return;
			}

			UObject* RecordingAssetData = AssetData.GetAsset();
			if (!RecordingAssetData)
			{
				UE_LOGF(LogLiveLinkHub, Warning, "Failed to import recording %ls", *AssetData.AssetName.ToString());
				return;
			}

			ULiveLinkRecording* ImportedRecording = CastChecked<ULiveLinkRecording>(RecordingAssetData);
			
			if (UE::IsSavingPackage(nullptr) && !ImportedRecording->IsFullyLoaded())
			{
				// With async saving we risk triggering checks during StaticFindObjectFast, even if the package we are loading isn't the one
				// being saved. This won't occur if the recording is fully loaded into memory already.
				UE_LOGF(LogLiveLinkHub, Warning, "Can't start recording because a package is saving");
				return;
			}
			
			HubPtr->GetPlaybackController()->PreparePlayback(ImportedRecording);
		}
	}
	/** Reacts to changes in the content directory to rescan it to update the recording list. */
	void OnContentDirectoryChanged(const TArray<struct FFileChangeData>& FileChangeData)
	{
		auto ScanChanges = [this, FileChangeData]()
		{
			if (IsEngineExitRequested() || !FLiveLinkHub::Get().IsValid())
			{
				return;
			}
				
			TArray<FString> FilePaths;
			FilePaths.Reserve(FileChangeData.Num());

			Algo::Transform(FileChangeData, FilePaths, [](const FFileChangeData& Data) { return Data.Filename; });
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
			AssetRegistry.ScanPathsSynchronous({ FilePaths }, /*bForceRescan=*/ true);
		};
		
		if (GEditor && GEditor->IsTimerManagerValid())
		{
			GEditor->GetTimerManager()->ClearTimer(DirectoryScanTimerHandle);
			GEditor->GetTimerManager()->SetTimer(DirectoryScanTimerHandle, ScanChanges, 1.f, false);
		}
		else
		{
			ScanChanges();
		}
	}

private:
	/** LiveLinkHub object that holds the different controllers. */
	TWeakPtr<FLiveLinkHub> LiveLinkHub;
	/** Delegate handle for the RegisterDirectoryChangedCallback_Handle callback. */
	FDelegateHandle DirectoryChangedHandle;
	/** Path to the content directory. Used to react to changes to that directory, such as new files being added. */
	FString ContentDirectory;

	/** Timer handle used for scanning directory changes. */
	FTimerHandle DirectoryScanTimerHandle;
};

#undef LOCTEXT_NAMESPACE /* LiveLinkHub.RecordingList */
