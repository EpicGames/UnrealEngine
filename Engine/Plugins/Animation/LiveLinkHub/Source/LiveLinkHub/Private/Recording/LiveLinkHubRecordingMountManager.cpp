// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubRecordingMountManager.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "LiveLinkHubLog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.RecordingMountManager"

FLiveLinkHubRecordingMountManager::FLiveLinkHubRecordingMountManager()
	: bIsMounted(false)
{
}

FLiveLinkHubRecordingMountManager::~FLiveLinkHubRecordingMountManager()
{
	// Ensure we clean up on destruction
	FText ErrorText;
	UnmountCustomDirectory(ErrorText);
}

bool FLiveLinkHubRecordingMountManager::MountCustomDirectory(const FString& InDirectoryPath, FString& OutMountPoint, FText& OutErrorText)
{
	// Validate the directory first
	if (!ValidateDirectory(InDirectoryPath, OutErrorText))
	{
		return false;
	}

	// Check if this directory is inside the project content dir - if so, no mount needed
	if (IsInsideProjectContent(InDirectoryPath))
	{
		const FString AbsolutePath = FPaths::ConvertRelativePathToFull(InDirectoryPath);
		FString FailureReason;
		if (!FPackageName::TryConvertFilenameToLongPackageName(AbsolutePath, OutMountPoint, &FailureReason))
		{
			OutErrorText = FText::Format(
				LOCTEXT("ConvertToPackageNameFailed", "Failed to convert content path to package name: {0}"),
				FText::FromString(FailureReason)
			);
			return false;
		}
		return true;
	}

	// Unmount existing mount point if any
	if (bIsMounted)
	{
		FText UnmountError;
		if (!UnmountCustomDirectory(UnmountError))
		{
			OutErrorText = FText::Format(
				LOCTEXT("UnmountFailed", "Failed to unmount existing directory: {0}"),
				UnmountError
			);
			return false;
		}
	}

	// Generate unique mount point
	const FString MountPoint = GenerateUniqueMountPoint(InDirectoryPath);

	// Ensure mount point ends with trailing slash for registration
	const FString MountPointWithSlash = MountPoint.EndsWith(TEXT("/")) ? MountPoint : MountPoint + TEXT("/");

	// Check if already mounted
	if (FPackageName::MountPointExists(MountPointWithSlash))
	{
		OutErrorText = FText::Format(
			LOCTEXT("MountPointExists", "Mount point {0} already exists"),
			FText::FromString(MountPointWithSlash)
		);
		return false;
	}

	// Register the mount point
	UE_LOGF(LogLiveLinkHub, Display, "Registering mount point %ls to %ls", *MountPointWithSlash, *InDirectoryPath);
	FPackageName::RegisterMountPoint(MountPointWithSlash, InDirectoryPath);

	// Store the state (without trailing slash for storage)
	CurrentMountPoint = MountPoint;
	CurrentMountedDirectory = InDirectoryPath;
	bIsMounted = true;
	OutMountPoint = MountPoint;

	UE_LOGF(LogLiveLinkHub, Display, "Successfully mounted %ls to %ls", *MountPoint, *InDirectoryPath);
	return true;
}

bool FLiveLinkHubRecordingMountManager::UnmountCustomDirectory(FText& OutErrorText)
{
	if (!bIsMounted)
	{
		// Not an error - just nothing to do
		return true;
	}

	// Add trailing slash for unregistration
	const FString MountPointWithSlash = CurrentMountPoint.EndsWith(TEXT("/")) ? CurrentMountPoint : CurrentMountPoint + TEXT("/");

	if (!FPackageName::MountPointExists(MountPointWithSlash))
	{
		OutErrorText = FText::Format(
			LOCTEXT("MountPointNotFound", "Mount point {0} does not exist"),
			FText::FromString(MountPointWithSlash)
		);
		bIsMounted = false;
		CurrentMountPoint.Empty();
		CurrentMountedDirectory.Empty();
		return false;
	}

	UE_LOGF(LogLiveLinkHub, Display, "Unregistering mount point %ls from %ls", *MountPointWithSlash, *CurrentMountedDirectory);
	FPackageName::UnRegisterMountPoint(MountPointWithSlash, CurrentMountedDirectory);

	bIsMounted = false;
	CurrentMountPoint.Empty();
	CurrentMountedDirectory.Empty();

	UE_LOGF(LogLiveLinkHub, Display, "Successfully unmounted recording directory");
	return true;
}

bool FLiveLinkHubRecordingMountManager::ValidateDirectory(const FString& InDirectoryPath, FText& OutErrorText) const
{
	if (InDirectoryPath.IsEmpty())
	{
		OutErrorText = LOCTEXT("EmptyPath", "Directory path cannot be empty");
		return false;
	}

	// Check if directory exists
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*InDirectoryPath))
	{
		OutErrorText = FText::Format(
			LOCTEXT("DirectoryNotFound", "Directory does not exist: {0}"),
			FText::FromString(InDirectoryPath)
		);
		return false;
	}

	// Validate write permissions by attempting to create a temp file
	const FString TestFilePath = FPaths::Combine(InDirectoryPath, TEXT("LiveLinkHub_WriteTest.tmp"));
	if (!FFileHelper::SaveStringToFile(TEXT("test"), *TestFilePath))
	{
		OutErrorText = FText::Format(
			LOCTEXT("NoWritePermission", "No write permission for directory: {0}"),
			FText::FromString(InDirectoryPath)
		);
		return false;
	}

	// Clean up test file
	IFileManager::Get().Delete(*TestFilePath, false, true);

	return true;
}

FString FLiveLinkHubRecordingMountManager::GenerateUniqueMountPoint(const FString& InDirectoryPath) const
{
	// Extract folder name from path
	FString FolderName = FPaths::GetCleanFilename(InDirectoryPath);

	// Sanitize folder name for use in mount point
	FolderName.ReplaceCharInline(TEXT(' '), TEXT('_'));
	FolderName.ReplaceCharInline(TEXT('-'), TEXT('_'));

	// Generate short UUID for uniqueness
	const FString ShortUUID = FGuid::NewGuid().ToString().Left(6).ToLower();

	// Construct mount point with /Game prefix (required by UE mount point system)
	// Format: /Game/LiveLinkRecordings_xxxxxx
	const FString BaseName = FString::Printf(TEXT("LiveLinkRecordings_%ls"), *ShortUUID);
	const FString MountPoint = FString::Printf(TEXT("/Game/%ls"), *BaseName);

	return MountPoint;
}

bool FLiveLinkHubRecordingMountManager::IsInsideProjectContent(const FString& InDirectoryPath) const
{
	const FString ProjectContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
	const FString FullDirectoryPath = FPaths::ConvertRelativePathToFull(InDirectoryPath);

	return FullDirectoryPath.StartsWith(ProjectContentDir);
}

#undef LOCTEXT_NAMESPACE
