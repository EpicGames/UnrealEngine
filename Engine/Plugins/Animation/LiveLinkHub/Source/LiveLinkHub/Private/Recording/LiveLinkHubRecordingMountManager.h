// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

/**
 * Manages custom mount points for LiveLink Hub recordings.
 * Allows users to record to folders outside the default Content directory.
 */
class FLiveLinkHubRecordingMountManager
{
public:
	FLiveLinkHubRecordingMountManager();
	~FLiveLinkHubRecordingMountManager();

	/**
	 * Mount a custom directory for recordings.
	 * @param InDirectoryPath Absolute path to the directory to mount
	 * @param OutMountPoint The generated mount point path (e.g., /Game/LiveLinkRecordings_eeaf31)
	 * @param OutErrorText Error message if mounting fails
	 * @return true if mounting succeeded
	 */
	bool MountCustomDirectory(const FString& InDirectoryPath, FString& OutMountPoint, FText& OutErrorText);

	/**
	 * Unmount the currently active custom directory.
	 * @param OutErrorText Error message if unmounting fails
	 * @return true if unmounting succeeded or no mount point was active
	 */
	bool UnmountCustomDirectory(FText& OutErrorText);

	/**
	 * Check if a custom directory is currently mounted.
	 */
	bool IsCustomDirectoryMounted() const { return bIsMounted; }

	/**
	 * Get the current mount point path (e.g., /Game/LiveLinkRecordings_eeaf31).
	 * Empty if no custom directory is mounted.
	 */
	const FString& GetMountPoint() const { return CurrentMountPoint; }

	/**
	 * Get the current mounted directory absolute path.
	 * Empty if no custom directory is mounted.
	 */
	const FString& GetMountedDirectory() const { return CurrentMountedDirectory; }

private:
	/**
	 * Validate that a directory path is valid for mounting.
	 * @param InDirectoryPath Absolute path to validate
	 * @param OutErrorText Error message if validation fails
	 * @return true if valid
	 */
	bool ValidateDirectory(const FString& InDirectoryPath, FText& OutErrorText) const;

	/**
	 * Generate a unique mount point name that doesn't conflict with existing /Game paths.
	 * @param InDirectoryPath The directory being mounted (used for base name)
	 * @return Mount point path like /Game/LiveLinkRecordings_eeaf31
	 */
	FString GenerateUniqueMountPoint(const FString& InDirectoryPath) const;

	/**
	 * Check if the directory is inside FPaths::ProjectContentDir().
	 * If true, no mounting is needed.
	 */
	bool IsInsideProjectContent(const FString& InDirectoryPath) const;

	/** Whether a custom directory is currently mounted */
	bool bIsMounted = false;

	/** Current mount point (e.g., /Game/LiveLinkRecordings_eeaf31) */
	FString CurrentMountPoint;

	/** Current mounted directory absolute path */
	FString CurrentMountedDirectory;
};
