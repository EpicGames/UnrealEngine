// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::LiveLinkHub::Private::RecordingVersions
{
	constexpr int32 InitialVersion = 1;
	/** Account for varying frame sizes and offsets. */
	constexpr int32 DynamicFrameSizes = 2;
	/** Timestamps serialized in bulk. */
	constexpr int32 BulkTimestamps = 3;
	/** Added oodle compression to serialized frame data. */
	constexpr int32 CompressionSupport = 4;

	/** The latest recording version. UPDATE IF ADDING NEW VERSIONS! */
	constexpr int32 Latest = CompressionSupport;
}
