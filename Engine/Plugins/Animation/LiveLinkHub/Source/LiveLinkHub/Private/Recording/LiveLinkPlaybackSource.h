// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"
#include "LiveLinkHubPlaybackSourceSettings.h"
#include "LiveLinkSourceSettings.h"

class ILiveLinkClient;

/**
 * Completely empty "source" displayed in the UI when playing back a recording.
 */
class FLiveLinkPlaybackSource : public ILiveLinkSource
{
public:
	FLiveLinkPlaybackSource() = default;

	FLiveLinkPlaybackSource(const FString& OriginalSourceName)
	{
		// Upgrading a recording has a side effect of re-saving the source with "Playback (SourceName)"
		// We want to strip the Playback prefix and only retain the source name, since "Playback" will be displayed in a different column.

		static const FString PlaybackPrefix = TEXT("Playback (");
		const int32 NameStartIndex = PlaybackPrefix.Len();

		if (OriginalSourceName.StartsWith(PlaybackPrefix))
		{
			int32 LastParenthesisIndex = 0;
			OriginalSourceName.FindLastChar(TEXT(')'), LastParenthesisIndex);
			
			if (LastParenthesisIndex - NameStartIndex > 0)
			{
				SourceName = *OriginalSourceName.Mid(NameStartIndex, LastParenthesisIndex - NameStartIndex);
			}
			else
			{
				SourceName = "Unnamed Source";
			}
		}
		else
		{
			SourceName = *OriginalSourceName;
		}
	}

	virtual ~FLiveLinkPlaybackSource() = default;

	//~ Begin ILiveLinkSource interface
	virtual bool CanBeDisplayedInUI() const override
	{
		return true;
	}
	
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override
	{
	}
	
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override
	{
	}
	
	virtual bool IsSourceStillValid() const override
	{
		return true; 
	}
	
	virtual bool RequestSourceShutdown() override
	{
		return true; 
	}

	virtual FText GetSourceType() const override
	{
		return FText::FromName(SourceName);
	}
	
	virtual FText GetSourceMachineName() const override
	{
		return FText();
	}
	
	virtual FText GetSourceStatus() const override
	{
		return NSLOCTEXT("LiveLinkPlaybackSource", "PlaybackSourceStatus", "Playback");
	}

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const 
	{
		return ULiveLinkHubPlaybackSourceSettings::StaticClass();
	}
	//~ End ILiveLinkSource interface

protected:
	/** Source name. */
	FName SourceName;
};
