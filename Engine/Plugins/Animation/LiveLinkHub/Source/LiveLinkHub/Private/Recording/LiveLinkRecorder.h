// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"

class ULiveLinkRecording;
struct FLiveLinkUAssetRecordingData;

/** Interface for handling serialization of livelink data. */
class ILiveLinkRecorder
{
public:
	virtual ~ILiveLinkRecorder() = default;

	/** Start recording livelink data. */
	virtual void StartRecording() = 0;
	/** Stop recording livelink data and prompt the user for a save location. */
	virtual void StopRecording() = 0;
	/** Record static data in the current recording. */
	virtual void RecordStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData) = 0;
	/** Record frame data in the current recording. */
	virtual void RecordFrameData(const FLiveLinkSubjectKey& SubjectKey, const FLiveLinkFrameDataStruct& FrameData) = 0;
	/** Return whether we are currently recording livelink data. */
	virtual bool IsRecording() const = 0;
	/** Returns whether we can begin recording. */
	virtual bool CanRecord(FText* OutErrorMessage = nullptr) const = 0;
	/** Directly provide the recording and save it. */
	virtual void SaveFromExistingRecording(ULiveLinkRecording* InRecording, const FString& InFilePath, bool bUpgradeInPlace) = 0;
	/** Creates a unique asset name and prompts the user for the recording name. */
	virtual bool GetSavePresetPackageName(FString& OutName, const FString& InDefaultName, bool bAlwaysOpenSaveAsDialog = false) const = 0;
};
