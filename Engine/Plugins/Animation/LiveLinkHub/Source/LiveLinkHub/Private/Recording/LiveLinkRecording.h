// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkPreset.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "UObject/Object.h"

#include "LiveLinkRecording.generated.h"

UCLASS(Abstract)
class ULiveLinkRecording : public UObject
{
public:
	GENERATED_BODY()

	ULiveLinkRecording()
	{
		RecordingPreset = CreateDefaultSubobject<ULiveLinkPreset>(TEXT("RecordingPreset"));
	}

	/** True if this asset has all data loaded. */
	virtual bool IsFullyLoaded() const { return false; }

	/** True while recording data is being written to bulk data. */
	virtual bool IsSavingRecordingData() const { return false; }

	/** Return the maximum frames for this recording. */
	virtual int32 GetMaxFrames() const { return 0; }

	/** Return the last timestamp of the recording. */
	virtual double GetLastTimestamp() const { return 0.0; }

	/** Return the last frame time. */
	virtual FQualifiedFrameTime GetLastFrameTime() const { return FQualifiedFrameTime(); }

	/** Retrieve the generalized framerate of this recording. */
	virtual FFrameRate GetGlobalFrameRate() const { return FFrameRate(); }

	/** Load the entire recording into memory. */
	virtual void LoadEntireRecording() { };

	/** If the entire recording is in the process of being loaded into memory. */
	virtual bool IsLoadingEntireRecording() const { return false; }

	/** LiveLink Preset used to save the initial state of the sources and subjects at the time of recording. */
	UPROPERTY(Instanced)
	TObjectPtr<ULiveLinkPreset> RecordingPreset = nullptr;

	/** Length of the recording. */
	UPROPERTY()
	double LengthInSeconds = 0.0;

	/** The framerate of the recording. */
	UPROPERTY()
	FFrameRate FrameRate;

	/** The saved version of this recording. */
	UPROPERTY(AssetRegistrySearchable)
	int32 SavedRecordingVersion;
};