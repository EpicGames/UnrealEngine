// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkRecordingRangeHelpers.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "StructUtils/StructView.h"
#include "Templates/SubclassOf.h"

class ULiveLinkRecording;

/** Frame that was read by the recording player*/
struct FLiveLinkRecordedFrame
{
	/** Recorded frame or static data. */
	FConstStructView Data;
	/** Subject that originally sent the data. */
	FLiveLinkSubjectKey SubjectKey;
	/** Role used to interpret the data (Only present with recorded static data). */
	TSubclassOf<ULiveLinkRole> LiveLinkRole;
	/** The frame index of this frame within a track. */
	int32 FrameIndex;
};

/** Object responsible for reading a livelink recording and providing the frames to the LiveLinkPlaybackController. */
class ILiveLinkRecordingPlayer
{
public:
	virtual ~ILiveLinkRecordingPlayer() = default;

	/** Initialize internal structures needed for playback of the recorded data. */
	virtual bool PreparePlayback(ULiveLinkRecording* Recording) = 0;

	/** Preform cleanup when exiting playback. */
	virtual void ShutdownPlayback() = 0;

	/** Fetch next frames at the provided playhead position. */
	virtual TArray<FLiveLinkRecordedFrame> FetchNextFramesAtTimestamp(const FQualifiedFrameTime& InFrameTime) = 0;

	/** Fetch previous frames at the provided playhead position. */
	virtual TArray<FLiveLinkRecordedFrame> FetchPreviousFramesAtTimestamp(const FQualifiedFrameTime& InFrameTime) = 0;
	
	/** Fetch next frames at the provided frame index. */
	virtual TArray<FLiveLinkRecordedFrame> FetchNextFramesAtIndex(const FQualifiedFrameTime& InFrameTime) = 0;

	/** Restart the recording from the beginning. */
	virtual void RestartPlayback(int32 InIndex = INDEX_NONE) = 0;

	/** Retrieve the first frame's frame rate information. */
	virtual FFrameRate GetInitialFramerate() = 0;

	/** Retrieve the currently buffered frames. */
	virtual UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32> GetBufferedFrameRanges() = 0;
};
