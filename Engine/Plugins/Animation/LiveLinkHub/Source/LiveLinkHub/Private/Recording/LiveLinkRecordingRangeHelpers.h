// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/QualifiedFrameTime.h"

namespace UE::LiveLinkHub::RangeHelpers::Private
{
	/** TArray<TRange<T>> */
	template<typename T>
	using TRangeArray = TArray<TRange<T>>;
	
	/** Gets the length of the range, accounting for exclusive or inclusive ranges. */
	template<typename T>
	T GetRangeLength(const TRange<T>& InRange)
	{
		const T LowerBound = InRange.GetLowerBoundValue();
		const T UpperBound = InRange.GetUpperBoundValue();

		if (InRange.GetUpperBound().IsExclusive())
		{
			return UpperBound - LowerBound;
		}
		return UpperBound - LowerBound + 1;
	}

	/** Gets the length of the range array. Does not account for inclusive ranges. */
	template<typename T>
	T GetRangeLength(const TRangeArray<T>& InRangeArray)
	{
		T MinLowerBound = TNumericLimits<T>::Max();
		T MaxUpperBound = TNumericLimits<T>::Min();
		for (const TRange<T>& InRange : InRangeArray)
		{
			const T LowerBound = InRange.GetLowerBoundValue();
			const T UpperBound = InRange.GetUpperBoundValue();
			if (LowerBound < MinLowerBound)
			{
				MinLowerBound = LowerBound;
			}
			if (UpperBound > MaxUpperBound)
			{
				MaxUpperBound = UpperBound;
			}
		}

		return MaxUpperBound - MinLowerBound + 1;
	}
	
	/** Makes an inclusive range. */
	template<typename T>
	TRange<T> MakeInclusiveRange(const T InStart, const T InEnd)
	{
		return TRange<T>(TRangeBound<T>::Inclusive(InStart), TRangeBound<T>::Inclusive(InEnd));
	}

	/** Convert a range from one frame rate to another. */
	template<typename T>
	TRange<T> ConvertRangeFrameRate(const TRange<T>& InRange, const FFrameRate& InFromFrameRate, const FFrameRate& InToFrameRate)
	{
		const FQualifiedFrameTime FromStart(InRange.GetLowerBoundValue(), InFromFrameRate);
		const FQualifiedFrameTime FromEnd(InRange.GetUpperBoundValue(), InFromFrameRate);

		const FFrameTime ToStart = FromStart.ConvertTo(InToFrameRate);
		const FFrameTime ToEnd = FromEnd.ConvertTo(InToFrameRate);

		return MakeInclusiveRange(ToStart.GetFrame().Value, ToEnd.GetFrame().Value);
	}
}
