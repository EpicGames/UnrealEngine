// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Delegates/DelegateCombinations.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IContentBrowserSingleton.h"
#include "IDesktopPlatform.h"
#include "Implementations/LiveLinkUAssetRecording.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubModule.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "Recording/LiveLinkHubPlaybackController.h"
#include "Recording/LiveLinkHubRecordingController.h"
#include "Recording/LiveLinkHubRecordingMountManager.h"
#include "Recording/LiveLinkRecording.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "Session/LiveLinkHubSessionData.h"
#include "UObject/SavePackage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.RecordingListView"

class SLiveLinkHubRecordingListView : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnImportRecording, const struct FAssetData&);
	
	SLATE_BEGIN_ARGS(SLiveLinkHubRecordingListView)
		{}
		SLATE_EVENT(FOnImportRecording, OnImportRecording)
	SLATE_END_ARGS()

	SLiveLinkHubRecordingListView()
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		OnAssetAddedHandle = AssetRegistry.OnAssetAdded().AddRaw(this, &SLiveLinkHubRecordingListView::OnAssetAdded);
		OnAssetRemovedHandle = AssetRegistry.OnAssetRemoved().AddRaw(this, &SLiveLinkHubRecordingListView::OnAssetRemoved);
	}

	virtual ~SLiveLinkHubRecordingListView() override
	{
		if (FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
		{
			IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			if (OnAssetAddedHandle.IsValid())
			{
				AssetRegistry.OnAssetAdded().Remove(OnAssetAddedHandle);
			}
			if (OnAssetRemovedHandle.IsValid())
			{
				AssetRegistry.OnAssetRemoved().Remove(OnAssetRemovedHandle);
			}
		}
	}
	
	//~ Begin SWidget interface
	void Construct(const FArguments& InArgs)
	{
		OnImportRecordingDelegate = InArgs._OnImportRecording;

		ChildSlot
		[
			SNew(SVerticalBox)
			// Top toolbar with folder picker button
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ChooseRecordingFolderButton", "Choose Recording Folder..."))
					.ToolTipText(LOCTEXT("ChooseRecordingFolderTooltip", "Select a custom folder for saving recordings"))
					.OnClicked(this, &SLiveLinkHubRecordingListView::OnChooseRecordingFolder)
					.IsEnabled(this, &SLiveLinkHubRecordingListView::CanChangeMountPoint)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ResetRecordingFolderButton", "Reset to Default"))
					.ToolTipText(LOCTEXT("ResetRecordingFolderTooltip", "Reset to the default project content folder"))
					.OnClicked(this, &SLiveLinkHubRecordingListView::OnResetRecordingFolder)
					.IsEnabled(this, &SLiveLinkHubRecordingListView::CanResetMountPoint)
				]
			]
			// Main content area with switcher
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this]() { return GetRecordingPickerVisibility() == EVisibility::Visible ? 0 : 1; })
				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Fill)
				[
					SAssignNew(BoxWidget, SBox)
					[
						CreateRecordingPicker()
					]
				]
			+ SWidgetSwitcher::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(GetNoAssetsWarningText())
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(4, 4))
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(4.0, 2.0))
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("RecordingsFolderButton", "Open Recordings Folder"))
						.OnClicked(this, &SLiveLinkHubRecordingListView::OnOpenRecordingsFolder)
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(4.0, 2.0))
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("RefreshRecordingsButton", "Refresh Recordings"))
						.OnClicked(this, &SLiveLinkHubRecordingListView::OnRefreshRecordings)
					]
				]
			]
		]
		// Bottom status bar showing current recording path
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RecordingPathLabel", "Recording Path: "))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SLiveLinkHubRecordingListView::GetCurrentRecordingPath)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
			]
		];
	}
	//~ End SWidget interface

private:
	/** Callback to notice the hub that we've selected a recording to play. */
	void OnImportRecording(const FAssetData& AssetData) const
	{
		OnImportRecordingDelegate.Execute(AssetData);
	}

	/** When an asset is added to the asset registry. */
	void OnAssetAdded(const FAssetData& InAssetData)
	{
		if (InAssetData.IsValid()
			&& (InAssetData.AssetClassPath == ULiveLinkUAssetRecording::StaticClass()->GetClassPathName()
				|| InAssetData.AssetClassPath == ULiveLinkRecording::StaticClass()->GetClassPathName()))
		{
			// Only update cache if the asset is in the current mount point
			if (!ShouldFilterAsset(InAssetData))
			{
				bAssetsAvailableCached = true;
				RefreshAssetViewDelegate.ExecuteIfBound(true);
			}
		}
	}

	/** When an asset is removed from the asset registry. */
	void OnAssetRemoved(const FAssetData& InAssetData)
	{
		if (InAssetData.IsValid()
			&& (InAssetData.AssetClassPath == ULiveLinkUAssetRecording::StaticClass()->GetClassPathName()
				|| InAssetData.AssetClassPath == ULiveLinkRecording::StaticClass()->GetClassPathName()))
		{
			bAssetsAvailableCached.Reset();
		}
	}

	/** The visibility status of the recording picker. */
	EVisibility GetRecordingPickerVisibility() const
	{
		// Cache the value initially, otherwise it is set on the asset added event.
		if (!bAssetsAvailableCached.IsSet())
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			TArray<FAssetData> AssetDataArray;
			FARFilter Filter = MakeAssetFilter();
			AssetRegistryModule.Get().GetAssets(MoveTemp(Filter), AssetDataArray);

			// Filter to only count assets directly in the current mount point (excludes subfolders)
			const bool bHasValidAsset = AssetDataArray.ContainsByPredicate([this](const FAssetData& AssetData)
			{
				return !ShouldFilterAsset(AssetData);
			});

			bAssetsAvailableCached = bHasValidAsset;
		}

		return bAssetsAvailableCached.GetValue() ? EVisibility::Visible : EVisibility::Hidden;
	}

	TArray<FAssetViewCustomColumn> GetCustomColumns() const
	{
		TArray<FAssetViewCustomColumn> ReturnValue;

		// "Last Modified" column: shows the file modification timestamp of the .uasset on disk.
		FAssetViewCustomColumn LastModifiedColumn;
		LastModifiedColumn.ColumnName = FName(TEXT("LastModified"));
		LastModifiedColumn.DataType = UObject::FAssetRegistryTag::TT_Chronological;
		LastModifiedColumn.DisplayName = LOCTEXT("LastModifiedName", "Last Modified");
		auto GetFileTimeStamp = [](const FAssetData& AssetData) -> FDateTime
		{
			const FString PackagePath = AssetData.PackageName.ToString();
			const FString FilePath = FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension());
			const FDateTime TimeStamp = IFileManager::Get().GetTimeStamp(*FilePath);
			if (TimeStamp != FDateTime::MinValue())
			{
				return TimeStamp;
			}

			// File may not exist on disk yet (e.g. newly created recording). Fall back to current time.
			return FDateTime::Now();
		};

		LastModifiedColumn.OnGetColumnData = FOnGetCustomAssetColumnData::CreateLambda(
			[GetFileTimeStamp](const FAssetData& AssetData, const FName& ColumnName) -> FString
			{
				return GetFileTimeStamp(AssetData).ToString();
			});
		LastModifiedColumn.OnGetColumnDisplayText = FOnGetCustomAssetColumnDisplayText::CreateLambda(
			[GetFileTimeStamp](const FAssetData& AssetData, const FName& ColumnName) -> FText
			{
				return FText::FromString(GetFileTimeStamp(AssetData).ToString(TEXT("%Y-%m-%d %H:%M")));
			});

		ReturnValue.Add(LastModifiedColumn);

		// "Version" column: shows the saved version number, with a warning icon when the recording is outdated.
		auto IsRecordingOutdated = [](const FAssetData& AssetData) -> bool
		{
			int32 AssetVersion;
			const bool bHasMetaTag = AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(ULiveLinkRecording, SavedRecordingVersion), AssetVersion);
			return !bHasMetaTag || AssetVersion < UE::LiveLinkHub::Private::RecordingVersions::Latest;
		};

		auto ColumnStringReturningLambda = [](const FAssetData& AssetData, const FName& ColumnName) -> FString
			{
				int32 AssetVersion;
				const bool bHasMetaTag = AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(ULiveLinkRecording, SavedRecordingVersion), AssetVersion);
				if (!bHasMetaTag || AssetVersion < UE::LiveLinkHub::Private::RecordingVersions::Latest)
				{
					return TEXT("Outdated");
				}

				return TEXT("Latest");
			};

		FAssetViewCustomColumn VersionColumn;
		VersionColumn.ColumnName = GET_MEMBER_NAME_CHECKED(ULiveLinkRecording, SavedRecordingVersion);
		VersionColumn.DataType = UObject::FAssetRegistryTag::TT_Alphabetical;
		VersionColumn.DisplayName = LOCTEXT("VersionName", "Version");
		VersionColumn.OnGetColumnData = FOnGetCustomAssetColumnData::CreateLambda(ColumnStringReturningLambda);
		VersionColumn.OnGetColumnDisplayText =
		FOnGetCustomAssetColumnDisplayText::CreateLambda(
			[ColumnStringReturningLambda](const FAssetData& AssetData, const FName& ColumnName)
			{
				return FText::FromString(ColumnStringReturningLambda(AssetData, ColumnName));
			});

		ReturnValue.Add(VersionColumn);
		return ReturnValue;
	}
	
	/** Creates the asset picker widget for selecting a recording. */
	TSharedRef<SWidget> CreateRecordingPicker(TOptional<FAssetData> AssetData = {})
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		
		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.CustomColumns = GetCustomColumns();
			AssetPickerConfig.SelectionMode = ESelectionMode::Single;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
			AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bShowBottomToolbar = true;
			AssetPickerConfig.bAutohideSearchBar = false;
			AssetPickerConfig.bAllowDragging = false;
			AssetPickerConfig.bCanShowClasses = false;
			AssetPickerConfig.bShowPathInColumnView = false;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.bSortByPathInColumnView = false;
			AssetPickerConfig.HiddenColumnNames.Add(TEXT("Path"));
			AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
			AssetPickerConfig.HiddenColumnNames.Add(TEXT("RevisionControl"));
			AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Tiny;
			AssetPickerConfig.AssetShowWarningText = GetNoAssetsWarningText();

			AssetPickerConfig.bForceShowEngineContent = true;
			AssetPickerConfig.bForceShowPluginContent = true;

			AssetPickerConfig.Filter = MakeAssetFilter();
			AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateRaw(this, &SLiveLinkHubRecordingListView::OnImportRecording);
			AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateRaw(this, &SLiveLinkHubRecordingListView::GetAssetContextMenu);
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateRaw(this, &SLiveLinkHubRecordingListView::ShouldFilterAsset);
			AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);

			if (AssetData.IsSet())
			{
				AssetPickerConfig.InitialAssetSelection = *AssetData;
			}
		}

		{
			AssetPicker = ContentBrowser.CreateAssetPicker(AssetPickerConfig);
			TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				AssetPicker.ToSharedRef()
			];

			MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
		}

		return MenuBuilder.MakeWidget();
	}

	/** Create a filter for available recording assets. */
	FARFilter MakeAssetFilter() const
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(ULiveLinkRecording::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		Filter.bRecursivePaths = true;
		// There shouldn't be recordings that exist in memory but not on disk. Necessary to properly register deleted assets.
		Filter.bIncludeOnlyOnDiskAssets = true;
		return Filter;
	}

	/** Filter out assets not directly in the current mount point (excludes subfolders). */
	bool ShouldFilterAsset(const FAssetData& AssetData) const
	{
		const FString MountPoint = GetRecordingMountPoint();
		const FString PackageFolder = FPackageName::GetLongPackagePath(AssetData.PackageName.ToString());
		return PackageFolder != MountPoint;
	}

	TSharedPtr<SWidget> GetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
	{
		if (SelectedAssets.Num() <= 0)
		{
			return nullptr;
		}

		const FAssetData& SelectedAsset = SelectedAssets[0];
		
		TWeakObjectPtr<UObject> SelectedAssetObject = SelectedAssets[0].GetAsset();
		if (!SelectedAssetObject.IsValid())
		{
			return nullptr;
		}

		FMenuBuilder MenuBuilder(true, MakeShared<FUICommandList>());

		MenuBuilder.BeginSection(TEXT("Recording"), LOCTEXT("RecordingSectionLabel", "Recording"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RenameRecordingLabel", "Rename"),
				LOCTEXT("RenameRecordingTooltip", "Rename the recording"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedAssetObject, this] ()
					{
						if (SelectedAssetObject.IsValid())
						{
							const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
							ContentBrowserModule.Get().ExecuteRename(AssetPicker);
						}
					}),
					FCanExecuteAction::CreateLambda([] () { return true; })
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteRecordingLabel", "Delete"),
				LOCTEXT("DeleteRecordingTooltip", "Delete the recording"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedAssetObject, this]()
					{
						if (SelectedAssetObject.IsValid())
						{
							IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

							TArray<FAssetData> Assets;
							AssetRegistry.GetAssetsByPackageName(*SelectedAssetObject->GetPackage()->GetPathName(), Assets);
						
							if (TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController = FLiveLinkHub::Get()->GetPlaybackController())
							{
								const TStrongObjectPtr<ULiveLinkRecording> PlaybackRecording = PlaybackController->GetRecording();
								if (PlaybackRecording.Get() == SelectedAssetObject)
								{
									/** Delete asset upon ejection being completed. */
									PlaybackController->Eject([Assets]() 
									{
										ObjectTools::DeleteAssets(Assets, false);
									});
								}
								else
								{
									ObjectTools::DeleteAssets(Assets, false);
								}
							}
						}
					}),
					FCanExecuteAction::CreateLambda([]() { return true; })
				)
			);
			
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DuplicateRecordingLabel", "Duplicate"),
				LOCTEXT("DuplicateRecordingTooltip", "Duplicate the recording"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Duplicate"),
				FUIAction(
					FExecuteAction::CreateLambda([this, SelectedAssetObject] ()
					{
						if (SelectedAssetObject.IsValid())
						{
							IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
							
							FString TargetName;
							FString TargetPackageName;
							IAssetTools::Get().CreateUniqueAssetName(SelectedAssetObject->GetOutermost()->GetName(), TEXT("_Copy"), TargetPackageName, TargetName);

							// Duplicate the asset.
							UObject* NewAsset = AssetTools.DuplicateAsset(TargetName, FPackageName::GetLongPackagePath(TargetPackageName), SelectedAssetObject.Get());
							FSavePackageArgs SavePackageArgs;
							SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
							SavePackageArgs.Error = GLog;

							// Save the package.
							const FString PackageFileName = FPackageName::LongPackageNameToFilename(TargetPackageName, FPackageName::GetAssetPackageExtension());
							UPackage::SavePackage(NewAsset->GetPackage(), NewAsset, *PackageFileName, MoveTemp(SavePackageArgs));

							// Unload the source recording data, as the bulk data would have been fully loaded to duplicate.
							const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
							const TStrongObjectPtr<ULiveLinkRecording> PlaybackRecording = LiveLinkHubModule.GetPlaybackController()->GetRecording();
							if (PlaybackRecording.Get() != SelectedAssetObject)
							{
								CastChecked<ULiveLinkUAssetRecording>(SelectedAssetObject)->UnloadRecordingData();
							}

							// There is no inherent way to update the selection of the asset picker, so instead we'll recreate one that is already selecting the new asset.
							BoxWidget->SetContent(CreateRecordingPicker(FAssetData{ NewAsset }));

							// It may take a few frames for the selection to fully update in the new picker, so give it ample time to do so before triggering the rename.
							GEditor->GetTimerManager()->SetTimer(TimerHandle, [this]() 
							{
								if (TimerHandle.IsValid())
								{
									if (const FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>("ContentBrowser"))
									{
										ContentBrowserModule->Get().ExecuteRename(AssetPicker);
									}
								}
							}, 0.3f, false);

							
						}
					}),
					FCanExecuteAction::CreateLambda([] () { return true; })
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenFileLocationLabel", "Open File Location..."),
				LOCTEXT("OpenFileLocationTooltip", "Open the folder containing this file"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedAssetObject] ()
					{
						if (SelectedAssetObject.IsValid())
						{
							const FString PackageName = SelectedAssetObject->GetPathName();
							const FString AssetFilePath = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
							const FString AbsoluteFilePath = FPaths::ConvertRelativePathToFull(AssetFilePath);
							const FString AssetDirectory = FPaths::GetPath(AbsoluteFilePath);
						
							FPlatformProcess::ExploreFolder(*AssetDirectory);
						}
					}),
					FCanExecuteAction::CreateLambda([SelectedAssetObject] () { return SelectedAssetObject.IsValid(); })
				)
			);
		}

		int32 SavedRecordingVersion = 0;
		const bool bHasRecordingValue = SelectedAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(ULiveLinkRecording, SavedRecordingVersion), SavedRecordingVersion);
		if (!bHasRecordingValue || SavedRecordingVersion < UE::LiveLinkHub::Private::RecordingVersions::Latest)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UpgradeRecording", "Upgrade Recording"),
				LOCTEXT("UpgradeRecordingTooltip", "Loads the entire recording into memory, converts it to the latest version, and saves to a new file."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Info"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedAssetObject] ()
					{
						if (SelectedAssetObject.IsValid())
						{
							const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
							LiveLinkHubModule.GetLiveLinkHub()->UpgradeAndSaveRecording(CastChecked<ULiveLinkUAssetRecording>(SelectedAssetObject.Get()));
						}
					}),
					FCanExecuteAction::CreateLambda([SelectedAssetObject] () { return SelectedAssetObject.IsValid(); })
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("UpgradeRecordingInPlace", "Upgrade Recording in place"),
				LOCTEXT("UpgradeRecordingInPlaceTooltip", "Loads the entire recording into memory, converts it to the latest version, and overwrites the recording with the updated version."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Info"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedAssetObject, this]()
					{
						if (SelectedAssetObject.IsValid())
						{
							if (TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController = FLiveLinkHub::Get()->GetPlaybackController())
							{
								const TStrongObjectPtr<ULiveLinkRecording> PlaybackRecording = PlaybackController->GetRecording();
								if (PlaybackRecording.Get() == SelectedAssetObject)
								{
									/** Upgrade asset once it's ejection is complete. */
									PlaybackController->Eject([SelectedAssetObject, this]()
									{
										constexpr bool bUpgradeInPlace = true;
										FLiveLinkHub::Get()->UpgradeAndSaveRecording(CastChecked<ULiveLinkUAssetRecording>(SelectedAssetObject.Get()), bUpgradeInPlace);
										RefreshAssetViewDelegate.ExecuteIfBound(true);
									});
								}
								else
								{
									constexpr bool bUpgradeInPlace = true;
									FLiveLinkHub::Get()->UpgradeAndSaveRecording(CastChecked<ULiveLinkUAssetRecording>(SelectedAssetObject.Get()), bUpgradeInPlace);
									RefreshAssetViewDelegate.ExecuteIfBound(true);
								}
							}
						}
					}),
					FCanExecuteAction::CreateLambda([SelectedAssetObject]() { return SelectedAssetObject.IsValid(); })
				)
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(TEXT("RecordingList"), LOCTEXT("RecordingListSectionLabel", "Recording List"));
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("RefreshRecordings", "Refresh Recordings"),
			LOCTEXT("RefreshRecordingsTooltip", "Rescan the directory list."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh"),
			FUIAction(
				FExecuteAction::CreateLambda([this] ()
				{
					const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
					AssetRegistryModule.Get().ScanPathsSynchronous({ GetRecordingMountPoint() }, true);
				}))
			);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}
	
	/** The text to display when no assets are found. */
	static FText GetNoAssetsWarningText()
	{
		return LOCTEXT("NoRecordings_Warning", "No recordings found. Press Record to create a new recording.");
	}

	/** Get the current recording mount point (custom or default /Game) */
	FString GetRecordingMountPoint() const
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = HubModule.GetSessionManager())
		{
			if (TSharedPtr<ILiveLinkHubSession> Session = SessionManager->GetCurrentSession())
			{
				if (ULiveLinkHubSessionData* SessionData = Cast<ULiveLinkHubSessionData>(Session->GetSessionData()))
				{
					if (SessionData->bUseCustomRecordingDirectory && !SessionData->CustomRecordingMountPoint.IsEmpty())
					{
						return SessionData->CustomRecordingMountPoint;
					}
				}
			}
		}
		return TEXT("/Game");
	}

	/**
	 * Get the user-selected custom recording directory (absolute filesystem path).
	 * SessionData is the source of truth: it covers both the registered-mount case and the
	 * content-subdirectory case (where the mount manager doesn't register a mount point).
	 */
	FString GetCustomRecordingDirectory() const
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = HubModule.GetSessionManager())
		{
			if (TSharedPtr<ILiveLinkHubSession> Session = SessionManager->GetCurrentSession())
			{
				if (ULiveLinkHubSessionData* SessionData = Cast<ULiveLinkHubSessionData>(Session->GetSessionData()))
				{
					if (SessionData->bUseCustomRecordingDirectory && !SessionData->CustomRecordingDirectory.IsEmpty())
					{
						return FPaths::ConvertRelativePathToFull(SessionData->CustomRecordingDirectory);
					}
				}
			}
		}
		return FString();
	}

	/** Get the current recording directory as text for display */
	FText GetCurrentRecordingPath() const
	{
		const FString CustomDirectory = GetCustomRecordingDirectory();
		if (!CustomDirectory.IsEmpty())
		{
			return FText::FromString(CustomDirectory);
		}

		// Default: project content directory
		return FText::FromString(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
	}

	/** Check if a custom folder is currently active */
	bool IsCustomFolderActive() const
	{
		return !GetCustomRecordingDirectory().IsEmpty();
	}

	/** Check if the mount point can be changed (not during playback or recording) */
	bool CanChangeMountPoint() const
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (TSharedPtr<FLiveLinkHubPlaybackController> PlaybackController = HubModule.GetPlaybackController())
		{
			if (PlaybackController->IsInPlayback())
			{
				return false;
			}
		}
		if (TSharedPtr<FLiveLinkHubRecordingController> RecordingController = HubModule.GetRecordingController())
		{
			if (RecordingController->IsRecording())
			{
				return false;
			}
		}
		return true;
	}

	/** Check if the mount point can be reset (custom folder active and not during playback or recording) */
	bool CanResetMountPoint() const
	{
		return IsCustomFolderActive() && CanChangeMountPoint();
	}

	/** Choose a custom recording folder */
	FReply OnChooseRecordingFolder()
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (!DesktopPlatform)
		{
			return FReply::Handled();
		}

		FString SelectedFolder;
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowHandle, TEXT("Choose Recording Folder"), FPaths::ProjectContentDir(), SelectedFolder))
		{
			const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
			if (FLiveLinkHubRecordingMountManager* MountManager = HubModule.GetRecordingMountManager())
			{
				FString OutMountPoint;
				FText ErrorText;
				if (MountManager->MountCustomDirectory(SelectedFolder, OutMountPoint, ErrorText))
				{
					// Update session data
					if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = HubModule.GetSessionManager())
					{
						if (TSharedPtr<ILiveLinkHubSession> Session = SessionManager->GetCurrentSession())
						{
							if (ULiveLinkHubSessionData* SessionData = Cast<ULiveLinkHubSessionData>(Session->GetSessionData()))
							{
								SessionData->bUseCustomRecordingDirectory = true;
								SessionData->CustomRecordingDirectory = SelectedFolder;
								SessionData->CustomRecordingMountPoint = OutMountPoint;
							}
						}
					}

					// Scan the new mount point for existing recordings
					IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
					AssetRegistry.ScanPathsSynchronous({ OutMountPoint }, true);

					// Trigger asset refresh
					RefreshAssetViewDelegate.ExecuteIfBound(true);

					// Clear the assets cache to force re-evaluation
					bAssetsAvailableCached.Reset();
				}
				else
				{
					UE_LOGF(LogLiveLinkHub, Warning, "Failed to mount custom recording directory: %ls", *ErrorText.ToString());
				}
			}
		}

		return FReply::Handled();
	}

	/** Reset to default recording folder */
	FReply OnResetRecordingFolder()
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (FLiveLinkHubRecordingMountManager* MountManager = HubModule.GetRecordingMountManager())
		{
			FText ErrorText;
			if (MountManager->UnmountCustomDirectory(ErrorText))
			{
				// Update session data
				if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = HubModule.GetSessionManager())
				{
					if (TSharedPtr<ILiveLinkHubSession> Session = SessionManager->GetCurrentSession())
					{
						if (ULiveLinkHubSessionData* SessionData = Cast<ULiveLinkHubSessionData>(Session->GetSessionData()))
						{
							SessionData->bUseCustomRecordingDirectory = false;
							SessionData->CustomRecordingDirectory.Empty();
							SessionData->CustomRecordingMountPoint.Empty();
						}
					}
				}

				// Scan the default /Game mount point for existing recordings
				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
				AssetRegistry.ScanPathsSynchronous({ TEXT("/Game") }, true);

				// Trigger asset refresh
				RefreshAssetViewDelegate.ExecuteIfBound(true);

				// Clear the assets cache to force re-evaluation
				bAssetsAvailableCached.Reset();
			}
			else
			{
				UE_LOGF(LogLiveLinkHub, Warning, "Failed to unmount custom recording directory: %ls", *ErrorText.ToString());
			}
		}

		return FReply::Handled();
	}

	/** Open the current recordings folder in explorer */
	FReply OnOpenRecordingsFolder()
	{
		const FString CustomDirectory = GetCustomRecordingDirectory();
		if (!CustomDirectory.IsEmpty())
		{
			FPlatformProcess::ExploreFolder(*CustomDirectory);
			return FReply::Handled();
		}

		// Default: open project content directory
		FPlatformProcess::ExploreFolder(*FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
		return FReply::Handled();
	}

	/** Refresh recordings by rescanning the current mount point */
	FReply OnRefreshRecordings()
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().ScanPathsSynchronous({ GetRecordingMountPoint() }, true);
		RefreshAssetViewDelegate.ExecuteIfBound(true);
		bAssetsAvailableCached.Reset();
		return FReply::Handled();
	}

private:
	/** Delegate used for noticing the hub that a recording was selected for playback. */
	FOnImportRecording OnImportRecordingDelegate;
	/** The asset picker used for selecting recordings. */
	TSharedPtr<SWidget> AssetPicker;
	/** Handle for when an asset is added to the asset registry. */
	FDelegateHandle OnAssetAddedHandle;
	/** Handle for when an asset is removed from the asset registry. */
	FDelegateHandle OnAssetRemovedHandle;
	/** Box widget used to hold the asset picker. */
	TSharedPtr<SBox> BoxWidget;
	/** Timer handle used for triggering a rename after duplicating a recording. */
	FTimerHandle TimerHandle;
	/** Delegate used to refresh the asset view. */
	FRefreshAssetViewDelegate RefreshAssetViewDelegate;
	/** True if there are recording assets that exist. */
	mutable TOptional<bool> bAssetsAvailableCached;
};

#undef LOCTEXT_NAMESPACE /* LiveLinkHub.RecordingListView */
