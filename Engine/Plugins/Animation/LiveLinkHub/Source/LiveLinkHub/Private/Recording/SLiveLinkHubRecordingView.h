// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubPlaybackController.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class FLiveLinkHub;

#define LOCTEXT_NAMESPACE "LiveLinkHub.RecordingView"


/**
 * Utility base class for all content that is displayed in an entire major tab.
 */
class SLiveLinkHubRecordingView : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(bool, FSimplePredicate);

	SLATE_BEGIN_ARGS(SLiveLinkHubRecordingView) {}
		SLATE_EVENT(FSimplePredicate, CanRecord)
		SLATE_EVENT(FSimplePredicate, IsRecording)
		SLATE_EVENT(FSimpleDelegate, OnStartRecording)
		SLATE_EVENT(FSimpleDelegate, OnStopRecording)
	SLATE_END_ARGS()

	/**
		* @param InArgs
		* @param InStatusBarId Unique ID needed for the status bar
		*/
	void Construct(const FArguments& InArgs)
	{
		CanRecordDelegate = InArgs._CanRecord;
		IsRecordingDelegate = InArgs._IsRecording;
		OnStartRecordingDelegate = InArgs._OnStartRecording;
		OnStopRecordingDelegate = InArgs._OnStopRecording;

		ChildSlot
		[
			SNew(SButton)
				.OnClicked(this, &SLiveLinkHubRecordingView::OnClickRecordButton)
				.IsEnabled(this, &SLiveLinkHubRecordingView::CanRecord)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Animation.Record"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(TAttribute<FText>::CreateSP(this, &SLiveLinkHubRecordingView::GetRecordingText))
						.TextStyle(FAppStyle::Get(), "NormalText")
					]
				]
		];
	}

	/**  Get the text shown in the Record button in the toolbar. */
	FText GetRecordingText() const
	{
		static FText RecordText = LOCTEXT("RecordButtonLabel", "Record");
		static FText RecordingText = LOCTEXT("RecordButtonRecordingLabel", "Recording...");
		return IsRecordingDelegate.Execute() ? RecordingText : RecordText;
	}

	/** Handler called when record button is clicked */
	FReply OnClickRecordButton()
	{
		if (IsRecordingDelegate.Execute())
		{
			OnStopRecordingDelegate.Execute();
		}
		else
		{
			OnStartRecordingDelegate.Execute();
		}

		return FReply::Handled(); 
	}

	/** Returns whether we're currently recording. */
	bool IsRecording() const
	{
		return IsRecordingDelegate.Execute();
	}

	bool CanRecord() const
	{
		return CanRecordDelegate.Execute();
	}

private:
	/** Delegate used to know if the hub can begin recording. */
	FSimplePredicate CanRecordDelegate;
	/** Delegate used to know if the hub is currently recording. */
	FSimplePredicate IsRecordingDelegate;
	/** Delegate used to notice the hub that we should start recording. */
	FSimpleDelegate OnStartRecordingDelegate;
	/** Delegate used to notice the hub that we should stop recording. */
	FSimpleDelegate OnStopRecordingDelegate;
};

#undef LOCTEXT_NAMESPACE /* LiveLinkHub.RecordingView */
