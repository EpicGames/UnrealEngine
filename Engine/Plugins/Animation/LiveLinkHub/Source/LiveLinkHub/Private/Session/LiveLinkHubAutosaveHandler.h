// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/MaxElement.h"
#include "Algo/Transform.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "LiveLinkHubSessionManager.h"
#include "Misc/Paths.h"

/** Utility class to trigger autosaves periodically. */
class FLiveLinkHubAutosaveHandler
{
public:
	inline static const FString AutosaveFileExtension = TEXT(".autosave");

	FLiveLinkHubAutosaveHandler()
	{
		FCoreDelegates::GetApplicationWillTerminateDelegate().AddRaw(this, &FLiveLinkHubAutosaveHandler::OnApplicationWillTerminate);
		FCoreDelegates::OnBeginFrame.AddRaw(this, &FLiveLinkHubAutosaveHandler::OnBeginFrame_GameThread);
		FLiveLinkHub::Get()->GetSessionManager()->OnActiveSessionChanged().AddRaw(this, &FLiveLinkHubAutosaveHandler::OnActiveSessionChanged);
		FLiveLinkHub::Get()->GetSessionManager()->OnSessionSaved().AddRaw(this, &FLiveLinkHubAutosaveHandler::Autosave);
	}
	
	~FLiveLinkHubAutosaveHandler()
	{
		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			LiveLinkHub->GetSessionManager()->OnSessionSaved().RemoveAll(this);
			LiveLinkHub->GetSessionManager()->OnActiveSessionChanged().RemoveAll(this);
		}

		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		FCoreDelegates::GetApplicationWillTerminateDelegate().RemoveAll(this);
	}

	/** Forcefully perform autosave. */
	void Autosave()
	{
		if (TSharedPtr<FLiveLinkHub> Hub = FLiveLinkHub::Get())
		{
			if (Hub->IsInPlayback())
			{
				// Prevent autosave in playback to avoid overriding a session with a recording's sources/subjects.
				return;
			}

			const FString PreviousFilePath = FPaths::Combine(AutosaveDirectory, FileName);

			FileName = FDateTime::Now().ToString() + AutosaveFileExtension;

			UE_LOGF(LogLiveLinkHub, Log, "Performing autosave to file %ls", *FileName);
			FString SavePath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("LiveLinkHub"), TEXT("Autosaves"), FileName);
			
			if (TSharedPtr<ILiveLinkHubSessionManager> SessionManager = Hub->GetSessionManager())
			{
				SessionManager->SaveCurrentSession(SavePath);
			}

			// One run of LiveLinkHub will have a single autosave file, so clear the previous file we were using since we have a new file.
			FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PreviousFilePath);

			LastAutosaveTimestamp = FPlatformTime::Seconds();

			TArray<FLiveLinkHubSessionFile> Autosaves = GetAutosaveFiles();
			if (Autosaves.Num() > GetDefault<ULiveLinkHubSettings>()->NumberOfAutosaveFilesToRetain && GetDefault<ULiveLinkHubSettings>()->NumberOfAutosaveFilesToRetain > 0)
			{
				FLiveLinkHubSessionFile* File = Algo::MinElementBy(Autosaves, &FLiveLinkHubSessionFile::LastModificationDate);

				UE_LOGF(LogLiveLinkHub, Log, "Reached max number of autosave files. Deleting autosave %ls", *File->FileName);
				FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*File->FilePath);
			}
		}
	}

	/** Restore an autosave based on a file path. */
	void RestoreAutosave(const FString& File)
	{
		FLiveLinkHub::Get()->GetSessionManager()->RestoreSession(File);
	}

	/** Get the list of autosave files, sorted from most recent to oldest. */
	TArray<FLiveLinkHubSessionFile> GetAutosaveFiles() const
	{
		TArray<FString> Files;
		IFileManager::Get().FindFiles(Files, *AutosaveDirectory, *AutosaveFileExtension);

		TArray<FLiveLinkHubSessionFile> Autosaves;
		Autosaves.Reserve(Files.Num());
		Algo::Transform(Files, Autosaves, [this](const FString& InFile) { return FLiveLinkHubSessionFile{ FPaths::Combine(AutosaveDirectory, InFile) }; });
		Algo::SortBy(Autosaves, &FLiveLinkHubSessionFile::LastModificationDate);
		Algo::Reverse(Autosaves);

		return Autosaves;
	}

private:
	/** Attempts to save the current session if enough time has passed since the last one. */
	void AttemptAutosave()
	{
		// todo: Attempt auto-save when changing session.
		const double CurrentTime = FPlatformTime::Seconds();

		if (CurrentTime - LastAutosaveTimestamp > GetDefault<ULiveLinkHubSettings>()->MinutesBetweenAutosave * 60)
		{
			Autosave();
		}
	}

	/** Trigger an autosave when we change session. */
	void OnActiveSessionChanged(const TSharedRef<ILiveLinkHubSession>&)
	{
		Autosave();
	}

	/** Game thread tick used to check if we need to autosave. */
	void OnBeginFrame_GameThread()
	{
		AttemptAutosave();
	}

	/** Try to do a last autosave when the app crashes. */
	void OnApplicationWillTerminate()
	{
		Autosave();
	}

private:
	/** Base directory for autosave files. */
	const FString AutosaveDirectory = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("LiveLinkHub"), TEXT("Autosaves"));
	/** Filename for auto-saves, based on the time the hub was opened. */
	FString FileName = FDateTime::Now().ToString() + AutosaveFileExtension;
	/** Timestamp of the last auto-save. */
	double LastAutosaveTimestamp = FPlatformTime::Seconds();
};
