// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Async/Async.h"
#include "Clients/LiveLinkHubProvider.h"
#include "Clients/LiveLinkHubUEClientInfo.h"
#include "Engine/SystemTimeTimecodeProvider.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "LiveLinkClient.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkHubSessionData.h"
#include "LiveLinkTimecodeProvider.h"
#include "LiveLinkTypes.h"
#include "LiveLinkVirtualSubject.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Settings/LiveLinkHubTimeAndSyncSettings.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubSession"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnClientAddedToSession, FLiveLinkHubClientId);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnClientRemovedFromSession, FLiveLinkHubClientId);


struct FLiveLinkHubSessionFile
{
	/** Session name. */
	FString FileName;
	/** Full file path to this config file. */
	FString FilePath;
	/** When the file was last modified. */
	FDateTime LastModificationDate;
	/** Whether this is an autosave. */
	bool bWasAutosaved = false;

	FLiveLinkHubSessionFile(const FString& InFile)
		: FileName(FPaths::GetBaseFilename(InFile))
		, FilePath(InFile)
		, LastModificationDate(IFileManager::Get().GetTimeStamp(*InFile))
	{
		// At some point we could also show what subjects we had in the tooltip, or some notes about the session.
		bWasAutosaved = FPaths::GetExtension(InFile) == TEXT("autosave");
	}
};

/**
 * Holds the state of the hub for an active session, can be swapped out with a different session using the session manager.
 */
class ILiveLinkHubSession
{
public:
	virtual ~ILiveLinkHubSession() = default;

	/** Add a client to this session. Note: Must be called from game thread. */
	virtual void AddClient(const FLiveLinkHubClientId& Client) = 0;

	/** Remove a client from this session. Note: Must be called from game thread. */
	virtual void RemoveClient(const FLiveLinkHubClientId& Client) = 0;

	/** Remove all clients from this session. Note: Must be called from game thread. */
	virtual void RemoveAllClients() = 0;

	/** Returns whether a client is in this session. */
	virtual bool IsClientInSession(const FLiveLinkHubClientId& Client) const = 0;

	/** Get the list of clients in this session (The list of clients that can receive data from the hub) */
	virtual TArray<FLiveLinkHubClientId> GetSessionClients() const = 0;

	/** Get the topology mode for this instance. */
	virtual ELiveLinkTopologyMode GetTopologyMode() const = 0;

	/** Set the topology mode for this instance. */
	virtual void SetTopologyMode(ELiveLinkTopologyMode Mode) = 0;

	/** Returns whether the session config allows sending data to this client, and optionally to a specific subject. */
	virtual bool ShouldTransmitToClient(const FLiveLinkHubClientId& Client, FLiveLinkSubjectName Subject = FLiveLinkSubjectName{}) const = 0;

	/** Get the session data object. */
	virtual ULiveLinkHubSessionData* GetSessionData() const = 0;
};

class FLiveLinkHubSession : public ILiveLinkHubSession, public TSharedFromThis<FLiveLinkHubSession>
{
public:
	FLiveLinkHubSession(FOnClientAddedToSession& OnClientAddedToSession, FOnClientRemovedFromSession& OnClientRemovedFromSession)
		: OnClientAddedToSessionDelegate(OnClientAddedToSession)
		, OnClientRemovedFromSessionDelegate(OnClientRemovedFromSession)
	{
		SessionData = TStrongObjectPtr<ULiveLinkHubSessionData>(NewObject<ULiveLinkHubSessionData>(GetTransientPackage()));
	}

	FLiveLinkHubSession(ULiveLinkHubSessionData* InSessionData, FOnClientAddedToSession& OnClientAddedToSession, FOnClientRemovedFromSession& OnClientRemovedFromSession)
		: OnClientAddedToSessionDelegate(OnClientAddedToSession)
		, OnClientRemovedFromSessionDelegate(OnClientRemovedFromSession)
	{
		SessionData = TStrongObjectPtr<ULiveLinkHubSessionData>(InSessionData);
	}

	virtual TArray<FLiveLinkHubClientId> GetSessionClients() const override
	{
		FReadScopeLock Locker(SessionDataLock);
		return CachedSessionClients.Array();
	}

	virtual void AddClient(const FLiveLinkHubClientId& Client) override
	{
		check(IsInGameThread());

		{
			if (TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkProvider())
			{
				if (TOptional<FLiveLinkHubUEClientInfo> ClientInfo = LiveLinkProvider->GetClientInfo(Client))
				{
					FWriteScopeLock Locker(SessionDataLock);
					CachedSessionClients.Add(Client);
				}

				const ULiveLinkHubTimeAndSyncSettings* Settings = GetDefault<ULiveLinkHubTimeAndSyncSettings>();

				if (Settings->bUseLiveLinkHubAsTimecodeSource)
				{
					LiveLinkProvider->UpdateTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile(), Client);
				}

				if (Settings->bUseLiveLinkHubAsCustomTimeStepSource)
				{
					LiveLinkProvider->UpdateCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile(), Client);
				}
			}
		}

		OnClientAddedToSessionDelegate.Broadcast(Client);
	}

	virtual void RemoveClient(const FLiveLinkHubClientId& Client) override
	{
		check(IsInGameThread());

		if (TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub").GetLiveLinkProvider())
		{
			LiveLinkProvider->ResetTimecodeSettings(Client);
			LiveLinkProvider->ResetCustomTimeStepSettings(Client);
			LiveLinkProvider->DisconnectClient(Client);
		}

		{
			FWriteScopeLock Locker(SessionDataLock);
			CachedSessionClients.Remove(Client);
		}

		OnClientRemovedFromSessionDelegate.Broadcast(Client);
	}

	virtual void RemoveAllClients() override
	{
		check(IsInGameThread());

		TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider();

		TSet<FLiveLinkHubClientId> Clients;
		{
			FReadScopeLock Locker(SessionDataLock);
			Clients = CachedSessionClients;
		}

		const bool bResetTimecode = GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsTimecodeSource;
		const bool bResetCustomTimeStep = GetDefault<ULiveLinkHubTimeAndSyncSettings>()->bUseLiveLinkHubAsCustomTimeStepSource;

		if (LiveLinkProvider)
		{
			for (const FLiveLinkHubClientId& ClientId : Clients)
			{
				if (bResetTimecode)
				{
					LiveLinkProvider->ResetTimecodeSettings(ClientId);
				}

				if (bResetCustomTimeStep)
				{
					LiveLinkProvider->ResetCustomTimeStepSettings(ClientId);
				}

				LiveLinkProvider->DisconnectClient(ClientId);
			}
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Warning, "LiveLinkProvider was already destroyed when RemoveAllClients was called");
		}

		{
			FWriteScopeLock Locker(SessionDataLock);
			CachedSessionClients.Reset();
		}

		for (const FLiveLinkHubClientId& ClientId : Clients)
		{
			OnClientRemovedFromSessionDelegate.Broadcast(ClientId);
		}
	}

	virtual bool IsClientInSession(const FLiveLinkHubClientId& Client) const override
	{
		FReadScopeLock Locker(SessionDataLock);
		return CachedSessionClients.Contains(Client);
	}

	virtual ELiveLinkTopologyMode GetTopologyMode() const override
	{
		FReadScopeLock Locker(SessionDataLock);
		return SessionData->TopologyMode;
	}

	virtual void SetTopologyMode(ELiveLinkTopologyMode Mode) override
	{
		{
			FWriteScopeLock Locker(SessionDataLock);
			SessionData->TopologyMode = Mode;
		}

		RemoveAllClients();

		if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider())
		{
			LiveLinkProvider->ConnectToAllDiscoveredClients();
		}
	}

	virtual bool ShouldTransmitToClient(const FLiveLinkHubClientId& Client, FLiveLinkSubjectName Subject = FLiveLinkSubjectName{}) const override
	{
		if (!IsClientInSession(Client))
		{
			return false;
		}

		{
			FReadScopeLock Locker(SessionDataLock);
			if (const FLiveLinkHubUEClientInfo* ClientInfo = Algo::FindBy(SessionData->Clients, Client , &FLiveLinkHubUEClientInfo::Id))
			{
				if (!ClientInfo->bEnabled || ((!Subject.IsNone() && ClientInfo->DisabledSubjects.Contains(Subject))))
				{
					return false;
				}
			}
		}

		return true;
	}

	virtual ULiveLinkHubSessionData* GetSessionData() const override
	{
		FReadScopeLock Locker(SessionDataLock);
		return SessionData.Get();
	}

	void AddRestoredClient(FLiveLinkHubUEClientInfo& InOutRestoredClientInfo)
	{
		if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider())
		{
			LiveLinkProvider->AddRestoredClient(InOutRestoredClientInfo);

			{
				FWriteScopeLock Locker(SessionDataLock);
				CachedSessionClients.Add(InOutRestoredClientInfo.Id);
			}
		}

		OnClientAddedToSessionDelegate.Broadcast(InOutRestoredClientInfo.Id);
	}

private:
	/** List of clients in the current session. These represent the unreal instances than can receive data from the hub. */
	TSet<FLiveLinkHubClientId> CachedSessionClients;

	/** Holds data for this session. */
	TStrongObjectPtr<ULiveLinkHubSessionData> SessionData;

	/** Delegate used to notice the hub about clients being added to this session. */
	FOnClientAddedToSession& OnClientAddedToSessionDelegate;

	/** Delegate used to notice the hub about clients being removed from this session. */
	FOnClientRemovedFromSession& OnClientRemovedFromSessionDelegate;

	/** Lock used to access the client config from different threads. */
	mutable FRWLock SessionDataLock;

	friend class FLiveLinkHubSessionManager;
};

#undef LOCTEXT_NAMESPACE
