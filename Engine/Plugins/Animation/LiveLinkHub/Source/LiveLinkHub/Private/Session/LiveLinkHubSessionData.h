// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Clients/LiveLinkHubUEClientInfo.h"
#include "CoreTypes.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"
#include "LiveLinkPresetTypes.h"
#include "Misc/Guid.h"

#include "LiveLinkHubSessionData.generated.h"


/** Live link hub session data that can be saved to disk. */
UCLASS()
class ULiveLinkHubSessionData : public UObject
{
public:
	GENERATED_BODY()

	ULiveLinkHubSessionData()
	{
		RecordingSessionName = TEXT("DefaultSession");
		RecordingSlateName = TEXT("DefaultSlate");
		RecordingTakeNumber = 1;
	}

	/** Live link hub sources. */
	UPROPERTY()
	TArray<FLiveLinkSourcePreset> Sources;

	/** Live link hub subjects. */
	UPROPERTY()
	TArray<FLiveLinkSubjectPreset> Subjects;

	/** Live link hub client info. */
	UPROPERTY()
	TArray<FLiveLinkHubUEClientInfo> Clients;

	/** Recording metadata - session name */
	UPROPERTY()
	FString RecordingSessionName;

	/** Recording metadata - slate name */
	UPROPERTY()
	FString RecordingSlateName;

	/** Recording metadata - take number */
	UPROPERTY()
	int32 RecordingTakeNumber;

	/** Whether to use a custom recording directory instead of the default project content folder */
	UPROPERTY()
	bool bUseCustomRecordingDirectory = false;

	/** Custom recording directory absolute path (only used if bUseCustomRecordingDirectory is true) */
	UPROPERTY()
	FString CustomRecordingDirectory;

	/** Mount point for custom recording directory (e.g., /Game/LiveLinkRecordings_eeaf31) */
	UPROPERTY()
	FString CustomRecordingMountPoint;

	/** If the hub should be a hub or a spoke. Hubs can only connect to UE instances, while spokes can connect to other hubs. */
	UPROPERTY()
	ELiveLinkTopologyMode TopologyMode = ELiveLinkTopologyMode::Hub;

	/** Generic / extensible storage for additional fields to be serialized with the session. */
	UPROPERTY(Instanced) // Instanced so that the contained types are serialized inline.
	TMap<TSubclassOf<ULiveLinkHubSessionExtraData>, TObjectPtr<ULiveLinkHubSessionExtraData>> ExtraDatas;

	/** ID of the process that created this session. Used by LiveLinkHub's CrashRecovery system to know which autosave files to consider. */
	UPROPERTY()
	uint32 ProcessId = 0;

public:
	ULiveLinkHubSessionExtraData* GetExtraData(TSubclassOf<ULiveLinkHubSessionExtraData> InExtraDataClass)
	{
		if (TObjectPtr<ULiveLinkHubSessionExtraData>* MaybeData = ExtraDatas.Find(InExtraDataClass))
		{
			check(*MaybeData);
			return *MaybeData;
		}

		return nullptr;
	}

	ULiveLinkHubSessionExtraData* GetOrCreateExtraData(TSubclassOf<ULiveLinkHubSessionExtraData> InExtraDataClass)
	{
		if (ULiveLinkHubSessionExtraData* ExistingData = GetExtraData(InExtraDataClass))
		{
			return ExistingData;
		}

		ULiveLinkHubSessionExtraData* NewData = NewObject<ULiveLinkHubSessionExtraData>(this, *InExtraDataClass);
		ExtraDatas.Add(InExtraDataClass, NewData);
		return NewData;
	}
};
