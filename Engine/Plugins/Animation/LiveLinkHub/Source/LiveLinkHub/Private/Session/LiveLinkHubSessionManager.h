// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubSession.h"

#include "Config/LiveLinkHubFileUtilities.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Engine/Engine.h"
#include "HAL/CriticalSection.h"
#include "IDesktopPlatform.h"
#include "LiveLinkHubClient.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkSourceSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Recording/LiveLinkHubRecordingController.h"
#include "Recording/LiveLinkHubRecordingMountManager.h"
#include "Settings/LiveLinkHubSettings.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub.SessionManager"

namespace UE::Private::SessionManagerUtils
{
	/** Returns whether the give filepath is pointing to a LiveLinkHub autosave file. */
	inline static bool IsAutosave(const FString& FilePath)
	{
		return FPaths::GetExtension(FilePath) == TEXT("autosave");
	}
}


DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveSessionChanged, const TSharedRef<ILiveLinkHubSession>& /**Active Session*/);

class ILiveLinkHubSessionManager
{
public:
	virtual ~ILiveLinkHubSessionManager() = default;

	/** Delegate called when a UE client is added to the current session, enabling it to receive data from the hub. */
	virtual FOnClientAddedToSession& OnClientAddedToSession() = 0;

	/** Delegate called when a UE client is removed from the current session, returning it to the list of discovered clients. */
	virtual FOnClientRemovedFromSession& OnClientRemovedFromSession() = 0;

	/** Delegate called when the active session changes, which will change the list of sources, subjects and clients. */
	virtual FOnActiveSessionChanged& OnActiveSessionChanged() = 0;

	/** Delegate called when the active session is saved. (Not triggered for autosaves) */
	virtual FSimpleMulticastDelegate& OnSessionSaved() = 0;

	/** Get the current session, which holds information about which sources, subjects and clients that should be enabled in the hub at the moment. */
	virtual TSharedPtr<ILiveLinkHubSession> GetCurrentSession() const = 0;

	/** Clear out the current session data and stat a new empty session. */
	virtual void NewSession() = 0;

	/** Prompt the user to save the current session in a given directory. */
	virtual void SaveSessionAs() = 0;

	/** Restore a session from file. If not provided, will prompt the user to browse for the file. */
	virtual void RestoreSession(FStringView InSessionPath = {}) = 0;

	/** Restore a session from an object in memory. */
	virtual void RestoreSession(ULiveLinkHubSessionData* RestoreSession) = 0;

	/** Read a session file and deserialize it to a UObject. */
	virtual TStrongObjectPtr<ULiveLinkHubSessionData> ReadSessionFile(FStringView InSessionPath) = 0;

	/** Save the current session. If not path is specified, the last save path will be used. */
	virtual void SaveCurrentSession(FString SavePath = TEXT("")) = 0;

	/** Returns whether the current session has as already been saved to disk before. */
	virtual bool CanSaveCurrentSession() const = 0;

	/** Returns the last used config path. */
	virtual const FString& GetLastConfigPath() const = 0;
};

class FLiveLinkHubSessionManager : public ILiveLinkHubSessionManager
	, public ILiveLinkRecordingSession
{
public:
	FLiveLinkHubSessionManager()
	{
		IModularFeatures::Get().RegisterModularFeature(ILiveLinkRecordingSession::GetModularFeatureName(), this);

		FScopeLock Lock(&CurrentSessionCS);
		CurrentSession = MakeShared<FLiveLinkHubSession>(OnClientAddedToSessionDelegate, OnClientRemovedFromSessionDelegate);
	}

	virtual ~FLiveLinkHubSessionManager() override
	{
		IModularFeatures::Get().UnregisterModularFeature(ILiveLinkRecordingSession::GetModularFeatureName(), this);
	}

	//~ Begin LiveLinkHubSessionManager
	virtual FOnClientAddedToSession& OnClientAddedToSession() override
	{
		check(IsInGameThread());
		return OnClientAddedToSessionDelegate;
	}

	virtual FOnClientRemovedFromSession& OnClientRemovedFromSession() override
	{
		check(IsInGameThread());
		return OnClientRemovedFromSessionDelegate;
	}

	virtual FOnActiveSessionChanged& OnActiveSessionChanged() override
	{
		check(IsInGameThread());
		return OnActiveSessionChangedDelegate;
	}

	virtual FSimpleMulticastDelegate& OnSessionSaved() override
	{
		check(IsInGameThread());
		return OnSessionSavedDelegate;
	}

	virtual void NewSession() override
	{
		ClearSession();
		LastConfigPath.Empty();
	}

	virtual void SaveSessionAs() override
	{
		const FString FileDescription = UE::LiveLinkHub::FileUtilities::Private::ConfigDescription;
		const FString Extensions = UE::LiveLinkHub::FileUtilities::Private::ConfigExtension;
		const FString FileTypes = FString::Printf(TEXT("%s (*.%s)|*.%s"), *FileDescription, *Extensions, *Extensions);

		const FString DefaultFile = UE::LiveLinkHub::FileUtilities::Private::ConfigDefaultFileName;

		TArray<FString> SaveFileNames;

		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const bool bFileSelected = DesktopPlatform->SaveFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("LiveLinkHubSaveAsTitle", "Save As").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_SAVE),
			DefaultFile,
			FileTypes,
			EFileDialogFlags::None,
			SaveFileNames);

		if (bFileSelected && SaveFileNames.Num() > 0)
		{
			SaveCurrentSession(SaveFileNames[0]);
		}
	}

	virtual TSharedPtr<ILiveLinkHubSession> GetCurrentSession() const override
	{
		FScopeLock Lock(&CurrentSessionCS);
		return CurrentSession;
	}

	virtual void SaveCurrentSession(FString SavePath = TEXT("")) override
	{
		if (SavePath.IsEmpty() && LastConfigPath.IsEmpty())
		{
			return;
		}

		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = LiveLinkHubModule.GetLiveLinkProvider();
		FLiveLinkHubClient* LiveLinkHubClient = static_cast<FLiveLinkHubClient*>(&ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));

		check(LiveLinkProvider);
		check(LiveLinkHubClient);

		TSharedPtr<FLiveLinkHubSession> CurrentSessionPtr;
		{
			FScopeLock Lock(&CurrentSessionCS);
			CurrentSessionPtr = CurrentSession;
		}

		ULiveLinkHubSessionData* LiveLinkHubSessionData = CastChecked<ULiveLinkHubSessionData>(StaticCastSharedPtr<FLiveLinkHubSession>(CurrentSessionPtr)->SessionData.Get());

		// Write sources
		LiveLinkHubSessionData->Sources.Empty();
		TArray<FGuid> SourceGuids = LiveLinkHubClient->GetSources();
		for (const FGuid& SourceGuid : SourceGuids)
		{
			LiveLinkHubSessionData->Sources.Add(LiveLinkHubClient->GetSourcePreset(SourceGuid, nullptr));
		}

		// Write subjects
		LiveLinkHubSessionData->Subjects.Empty();
		TArray<FLiveLinkSubjectKey> Subjects = LiveLinkHubClient->GetSubjects(true, true);
		for (const FLiveLinkSubjectKey& Subject : Subjects)
		{
			LiveLinkHubSessionData->Subjects.Add(LiveLinkHubClient->GetSubjectPreset(Subject, nullptr));
		}

		// Write clients
		// FIXME?: We don't clear this one first like the others?
		const TMap<FLiveLinkHubClientId, FLiveLinkHubUEClientInfo>& ClientMap = LiveLinkProvider->GetClientsMap();
		for (const TTuple<FLiveLinkHubClientId, FLiveLinkHubUEClientInfo>& ClientKeyVal : ClientMap)
		{
			LiveLinkHubSessionData->Clients.Add(ClientKeyVal.Value);
		}

		// Write extra data
		LiveLinkHubSessionData->ExtraDatas.Empty();
		TArray<ILiveLinkHubSessionExtraDataHandler*> ExtraDataHandlers =
			ILiveLinkHubSessionExtraDataHandler::GetRegisteredHandlers();
		for (ILiveLinkHubSessionExtraDataHandler* Handler : ExtraDataHandlers)
		{
			TSubclassOf<ULiveLinkHubSessionExtraData> ExtraDataClass = Handler->GetExtraDataClass();
			ULiveLinkHubSessionExtraData* ExtraData = LiveLinkHubSessionData->GetOrCreateExtraData(ExtraDataClass);
			Handler->OnExtraDataSessionSaving(ExtraData);
		}

		LiveLinkHubSessionData->ProcessId = FPlatformProcess::GetCurrentProcessId();

		// Save recording mount point state
		if (FLiveLinkHubRecordingMountManager* MountManager = LiveLinkHubModule.GetRecordingMountManager())
		{
			LiveLinkHubSessionData->bUseCustomRecordingDirectory = MountManager->IsCustomDirectoryMounted();
			if (MountManager->IsCustomDirectoryMounted())
			{
				LiveLinkHubSessionData->CustomRecordingDirectory = MountManager->GetMountedDirectory();
				LiveLinkHubSessionData->CustomRecordingMountPoint = MountManager->GetMountPoint();
			}
			else
			{
				LiveLinkHubSessionData->CustomRecordingDirectory.Empty();
				LiveLinkHubSessionData->CustomRecordingMountPoint.Empty();
			}
		}

		if (SavePath.IsEmpty())
		{
			// This means we're saving the current session.
			SavePath = LastConfigPath;
		}

		UE::LiveLinkHub::FileUtilities::Private::SaveConfig(LiveLinkHubSessionData, SavePath);

		if (!UE::Private::SessionManagerUtils::IsAutosave(SavePath))
		{
			// Don't cache the last config path for autosaves since we don't really want the user to save in that folder.
			LastConfigPath = SavePath;
			GetMutableDefault<ULiveLinkHubUserSettings>()->CacheRecentConfig(SavePath);
			OnSessionSavedDelegate.Broadcast();
		}
	}

	virtual void RestoreSession(FStringView InSessionPath /* = {} */) override
	{
		// If no path was provided, prompt the user to browse for a session file.
		TArray<FString> OpenFileNames;
		if (InSessionPath.IsEmpty())
		{
			const FString FileDescription = UE::LiveLinkHub::FileUtilities::Private::ConfigDescription;
			const FString Extensions = UE::LiveLinkHub::FileUtilities::Private::ConfigExtension;
			const FString FileTypes = FString::Printf(TEXT("%s (*.%s)|*.%s"), *FileDescription, *Extensions, *Extensions);

			const FString DefaultFile = UE::LiveLinkHub::FileUtilities::Private::ConfigDefaultFileName;

			FString DefaultOpenPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
			if (!GetDefault<ULiveLinkHubUserSettings>()->LastConfigDirectory.IsEmpty())
			{
				DefaultOpenPath = GetDefault<ULiveLinkHubUserSettings>()->LastConfigDirectory;
			}

			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
			OpenSessionTimestampsSeconds = FPlatformTime::Seconds();
			const bool bFileSelected = DesktopPlatform->OpenFileDialog(
				ParentWindowWindowHandle,
				LOCTEXT("LiveLinkHubOpenTitle", "Open").ToString(),
				DefaultOpenPath,
				DefaultFile,
				FileTypes,
				EFileDialogFlags::None,
				OpenFileNames);

			if (bFileSelected && OpenFileNames.Num() > 0)
			{
				const FString& FilePath = OpenFileNames[0];
				InSessionPath = FilePath;

				if (!UE::Private::SessionManagerUtils::IsAutosave(FilePath))
				{
					GetMutableDefault<ULiveLinkHubUserSettings>()->CacheRecentConfig(FilePath);
				}
			}
		}

		if (!InSessionPath.IsEmpty())
		{
			if (!UE::Private::SessionManagerUtils::IsAutosave(FString{ InSessionPath }))
			{
				GetMutableDefault<ULiveLinkHubUserSettings>()->CacheRecentConfig(FString{ InSessionPath });
			}

			// Certain sources may take time to clean up. If they don't complete in time then the new config being loaded may not create
			// duplicate sources correctly. There should be errors in the logs of the sources that failed to remove or were unable to be added.
			constexpr bool bWaitForSourceRemoval = true;
			ClearSession(bWaitForSourceRemoval);

			if (ULiveLinkHubSessionData* SessionData = UE::LiveLinkHub::FileUtilities::Private::LoadConfig(FString{ InSessionPath }))
			{
				LastConfigPath = InSessionPath;
				RestoreSession(SessionData);
			}
		}
	}

	virtual void RestoreSession(ULiveLinkHubSessionData* SessionData)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		FLiveLinkHubClient* LiveLinkHubClient = static_cast<FLiveLinkHubClient*>(&ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));

		const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = LiveLinkHubModule.GetLiveLinkProvider();

		check(LiveLinkHubClient);
		check(LiveLinkProvider);

		if (SessionData)
		{
			for (const FLiveLinkSourcePreset& SourcePreset : SessionData->Sources)
			{
				LiveLinkHubClient->CreateSource(SourcePreset);
				// Ensure stored source settings persist. CreateSource will call Source->InitializeSettings, which passes in
				// a mutable settings object. Some sources may set "default" values on the settings object overriding the
				// saved values from the config. We want to prevent that behavior, but we still have to call InitializeSettings, because
				// other sources may set internal values based on the current settings' values, which is behavior we want to keep.
				if (ULiveLinkSourceSettings* PresetSettings = SourcePreset.Settings.Get())
				{
					if (ULiveLinkSourceSettings* CreatedSettings = LiveLinkHubClient->GetSourceSettings(SourcePreset.Guid))
					{
						UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
						CopyParams.bDoDelta = false;
						UEngine::CopyPropertiesForUnrelatedObjects(PresetSettings, CreatedSettings, CopyParams);
					}
				}
			}

			for (const FLiveLinkSubjectPreset& SubjectPreset : SessionData->Subjects)
			{
				LiveLinkHubClient->CreateSubject(SubjectPreset);
			}
		}

		TSharedPtr<FLiveLinkHubSession> CurrentSessionPtr;
		{
			FScopeLock Lock(&CurrentSessionCS);
			CurrentSessionPtr = CurrentSession = MakeShared<FLiveLinkHubSession>(SessionData, OnClientAddedToSessionDelegate, OnClientRemovedFromSessionDelegate);
		}

		for (FLiveLinkHubUEClientInfo& Client : SessionData->Clients)
		{
			CurrentSessionPtr->AddRestoredClient(Client);
		}

		// Restore extra data
		TArray<ILiveLinkHubSessionExtraDataHandler*> ExtraDataHandlers =
			ILiveLinkHubSessionExtraDataHandler::GetRegisteredHandlers();
		for (ILiveLinkHubSessionExtraDataHandler* Handler : ExtraDataHandlers)
		{
			TSubclassOf<ULiveLinkHubSessionExtraData> ExtraDataClass = Handler->GetExtraDataClass();
			ULiveLinkHubSessionExtraData* ExtraData = SessionData->GetExtraData(ExtraDataClass);
			// Might be null.
			Handler->OnExtraDataSessionLoaded(ExtraData);
		}

		// Restore recording mount point
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (FLiveLinkHubRecordingMountManager* MountManager = HubModule.GetRecordingMountManager())
		{
			if (SessionData->bUseCustomRecordingDirectory && !SessionData->CustomRecordingDirectory.IsEmpty())
			{
				FString OutMountPoint;
				FText ErrorText;
				if (!MountManager->MountCustomDirectory(SessionData->CustomRecordingDirectory, OutMountPoint, ErrorText))
				{
					UE_LOGF(LogLiveLinkHub, Warning, "Failed to restore custom recording directory '%ls': %ls",
						*SessionData->CustomRecordingDirectory, *ErrorText.ToString());
					// Clear the flag so we fall back to default directory
					SessionData->bUseCustomRecordingDirectory = false;
				}
			}
			else
			{
				// Ensure any existing mount point is cleaned up
				FText ErrorText;
				MountManager->UnmountCustomDirectory(ErrorText);
			}
		}

		OnActiveSessionChangedDelegate.Broadcast(CurrentSessionPtr.ToSharedRef());

		const ULiveLinkHubTimeAndSyncSettings* Settings = GetDefault<ULiveLinkHubTimeAndSyncSettings>();
		if (Settings->bUseLiveLinkHubAsTimecodeSource)
		{
			LiveLinkProvider->UpdateTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile());
		}
		if (Settings->bUseLiveLinkHubAsCustomTimeStepSource)
		{
			LiveLinkProvider->UpdateCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile());
		}

		if (GetDefault<ULiveLinkHubSettings>()->bRemoveInvalidSubjectsAfterLoadingSession)
		{
			constexpr double ActualWaitTime = 1.5;

			// This is odd, but since we're blocking the game thread for a while to choose a session file, the internal time of the TimerManager hasn't been updated yet.
			// In order for our timer to have the correct start time, we need to offset our time correctly so that it matches the internal time, else our timer would execute immediately.
			double AdjustedWaitTime = FPlatformTime::Seconds() - OpenSessionTimestampsSeconds + ActualWaitTime;

			FTimerHandle Handle;
			GEditor->GetTimerManager()->SetTimer(Handle, FTimerDelegate::CreateRaw(this, &FLiveLinkHubSessionManager::CullUnresponsiveSubjects), AdjustedWaitTime, false);
		}
	}

	virtual TStrongObjectPtr<ULiveLinkHubSessionData> ReadSessionFile(FStringView InSessionPath) override
	{
		// Return a StrongObjectPtr since this object lives in the transient package.
		TStrongObjectPtr<ULiveLinkHubSessionData> SessionData;
		if (ULiveLinkHubSessionData* DataPtr = UE::LiveLinkHub::FileUtilities::Private::LoadConfig(FString{ InSessionPath }))
		{
			SessionData = TStrongObjectPtr{ DataPtr };
		}

		return SessionData;
	}

	virtual bool CanSaveCurrentSession() const override
	{
		return !LastConfigPath.IsEmpty();
	}

	virtual const FString& GetLastConfigPath() const override
	{
		return LastConfigPath;
	}	
	//~ End LiveLinkHubSessionManager

	//~ Begin ILiveLinkRecordingSession interface
	virtual FString GetSessionName() const override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			return CurrentSession->SessionData->RecordingSessionName;
		}

		return FString();
	}

	virtual FString GetSlateName() const override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			return CurrentSession->SessionData->RecordingSlateName;
		}

		return FString();
	}

	virtual int32 GetTakeNumber() const override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			return CurrentSession->SessionData->RecordingTakeNumber;
		}

		return -1;
	}

	virtual bool SetSessionName(FStringView InSessionName) override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			CurrentSession->SessionData->RecordingSessionName = InSessionName;
			OnSessionNameChangedDelegate.Broadcast(InSessionName);
			return true;
		}

		return false;
	}

	virtual bool SetSlateName(FStringView InSlateName) override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			CurrentSession->SessionData->RecordingSlateName = InSlateName;
			OnSlateNameChangedDelegate.Broadcast(InSlateName);
			return true;
		}

		return false;
	}

	virtual bool SetTakeNumber(int32 InTakeNumber) override
	{
		FScopeLock Lock(&CurrentSessionCS);
		if (ensure(CurrentSession))
		{
			CurrentSession->SessionData->RecordingTakeNumber = InTakeNumber;
			OnTakeNumberChangedDelegate.Broadcast(InTakeNumber);
			return true;
		}

		return false;
	}

	virtual FOnSessionStringChanged& OnSessionNameChanged() override
	{
		return OnSessionNameChangedDelegate;
	}

	virtual FOnSessionStringChanged& OnSlateNameChanged() override
	{
		return OnSlateNameChangedDelegate;
	}

	virtual FOnSessionIntChanged& OnTakeNumberChanged() override
	{
		return OnTakeNumberChangedDelegate;
	}

	virtual bool IsRecording() const override
	{
		const FLiveLinkHubModule& LiveLinkHubModule =
			FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

		if (TSharedPtr<FLiveLinkHubRecordingController> RecordingController = LiveLinkHubModule.GetRecordingController())
		{
			return RecordingController->IsRecording();
		}

		return false;
	}

	virtual FSimpleMulticastDelegate& OnRecordingStarted() override
	{
		return OnRecordingStartedDelegate;
	}

	virtual FSimpleMulticastDelegate& OnRecordingStopped() override
	{
		return OnRecordingStoppedDelegate;
	}

	virtual bool CanRecord() const override
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

		if (TSharedPtr<FLiveLinkHubRecordingController> RecordingController = HubModule.GetRecordingController())
		{
			return RecordingController->CanRecord();
		}

		return false;
	}

	virtual bool StartRecording() override
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

		if (TSharedPtr<FLiveLinkHubRecordingController> RecordingController = HubModule.GetRecordingController())
		{
			RecordingController->StartRecording();
			return RecordingController->IsRecording();
		}

		return false;
	}

	virtual bool StopRecording() override
	{
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");

		if (TSharedPtr<FLiveLinkHubRecordingController> RecordingController = HubModule.GetRecordingController())
		{
			if (RecordingController->IsRecording())
			{
				RecordingController->StopRecording();
				return !RecordingController->IsRecording();
			}
		}

		return false;
	}
	//~ End ILiveLinkRecordingSession interface

private:
	/** Clear the hub data contained in the current session, resetting the hub to its default state. */
	void ClearSession(bool bWaitForSourceRemoval = false)
	{
		// Unmount any custom recording directory
		const FLiveLinkHubModule& HubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		if (FLiveLinkHubRecordingMountManager* MountManager = HubModule.GetRecordingMountManager())
		{
			FText ErrorText;
			MountManager->UnmountCustomDirectory(ErrorText);
		}

		FLiveLinkHubClient* LiveLinkHubClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		check(LiveLinkHubClient);
		
		const float TimeToWaitForRemoval = bWaitForSourceRemoval ? GetDefault<ULiveLinkHubSettings>()->SourceMaxCleanupTime : 0.f;
		for (const FLiveLinkSubjectKey& Subject : LiveLinkHubClient->GetSubjects(true, true))
		{
			// Make sure we clear all subjects (including virtual subjects).
			LiveLinkHubClient->RemoveVirtualSubject(Subject);
		}

		const bool bRemovedAllSources = LiveLinkHubClient->RemoveAllSourcesWithTimeout(TimeToWaitForRemoval);
		
		if (!bRemovedAllSources && bWaitForSourceRemoval)
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Could not remove all existing sources in time. Sources may still be getting cleaned up.");
		}
		
		TSharedPtr<FLiveLinkHubSession> CurrentSessionPtr;
		{
			FScopeLock Lock(&CurrentSessionCS);
			CurrentSessionPtr = CurrentSession = MakeShared<FLiveLinkHubSession>(OnClientAddedToSessionDelegate, OnClientRemovedFromSessionDelegate);
		}

		// Issue nullptr extra data load events as a convenience; handlers probably still want to be aware of the session load event.
		TArray<ILiveLinkHubSessionExtraDataHandler*> ExtraDataHandlers =
			ILiveLinkHubSessionExtraDataHandler::GetRegisteredHandlers();
		for (ILiveLinkHubSessionExtraDataHandler* Handler : ExtraDataHandlers)
		{
			Handler->OnExtraDataSessionLoaded(nullptr);
		}

		OnActiveSessionChangedDelegate.Broadcast(CurrentSessionPtr.ToSharedRef());
	}

	/** Remove subjects that haven't received data in a while. */
	void CullUnresponsiveSubjects()
	{
		FLiveLinkHubClient* LiveLinkHubClient = static_cast<FLiveLinkHubClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		if (LiveLinkHubClient)
		{
			// We probably don't want to remove subjects that were explicitly marked as disabled.
			constexpr bool bDisabledSubjects = false;
			constexpr bool bVirtualSubjects = false;

			TArray<FLiveLinkSubjectKey> Subjects = LiveLinkHubClient->GetSubjects(bDisabledSubjects, bVirtualSubjects);
			for (const FLiveLinkSubjectKey& SubjectKey : Subjects)
			{
				if (LiveLinkHubClient->GetSubjectState(SubjectKey.SubjectName) == ELiveLinkSubjectState::Unresponsive)
				{
					LiveLinkHubClient->RemoveSubject_AnyThread(SubjectKey);
				}
			}
		}
	}

private:
	/** Session that holds the current configuration of the hub (Clients, sources, subjects). */
	TSharedPtr<FLiveLinkHubSession> CurrentSession;

	/** Last path where we saved a session config file. */
	FString LastConfigPath;

	/** Delegate triggered when a client is added to the current session. */
	FOnClientAddedToSession OnClientAddedToSessionDelegate;

	/** Delegate triggered when a client is removed from the current session. */
	FOnClientRemovedFromSession OnClientRemovedFromSessionDelegate;

	/** Delegate triggered when the current session is changed. */
	FOnActiveSessionChanged OnActiveSessionChangedDelegate;

	/** Delegate called when the session is saved. */
	FSimpleMulticastDelegate OnSessionSavedDelegate;

	/** Delegate triggered when the recording session name is changed. */
	FOnSessionStringChanged OnSessionNameChangedDelegate;

	/** Delegate triggered when the recording slate name is changed. */
	FOnSessionStringChanged OnSlateNameChangedDelegate;

	/** Delegate triggered when the recording take number is changed. */
	FOnSessionIntChanged OnTakeNumberChangedDelegate;

	/** Delegate triggered when recording begins. */
	FSimpleMulticastDelegate OnRecordingStartedDelegate;

	/** Delegate triggered when recording ends. */
	FSimpleMulticastDelegate OnRecordingStoppedDelegate;

	/** Timestamp of when the open session button was clicked. */
	double OpenSessionTimestampsSeconds = 0.0;

	/** Critical section used to synchronize access to the current session. */
	mutable FCriticalSection CurrentSessionCS;
};

#undef LOCTEXT_NAMESPACE /*LiveLinkHub.SessionManager*/
