// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "CoreMinimal.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "LiveLinkHubSettings.h"
#include "PropertyHandle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

/**
 * Customization for the CustomTimeStep settings.
 */
class FLiveLinkHubClientTextFilterCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkHubClientTextFilterCustomization>();
	}

	virtual bool ShouldInlineKey() const override
	{
		return true;
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		if (!PropertyHandle->IsValidHandle())
		{
			return;
		}

		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
		TObjectPtr<class UScriptStruct> Struct = CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty())->Struct;

		check(Struct);
		check(Struct == FLiveLinkHubClientTextFilter::StaticStruct());

		PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([]() { GetMutableDefault<ULiveLinkHubUserSettings>()->PostUpdateClientFilters(); }));

		TSharedPtr<IPropertyHandle> BehaviorHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkHubClientTextFilter, Behavior));
		TSharedPtr<IPropertyHandle> TypeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkHubClientTextFilter, Type));
		TSharedPtr<IPropertyHandle> TextHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkHubClientTextFilter, Text));
		TSharedPtr<IPropertyHandle> ProjectHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkHubClientTextFilter, Project));

		HeaderRow.NameContent()
			.MaxDesiredWidth(20) // use DPI?
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(2.0))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					BehaviorHandle->CreatePropertyValueWidget()
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(2.0))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					TypeHandle->CreatePropertyValueWidget()
				]
				+ SHorizontalBox::Slot()
				.MinWidth(40.0)
				[
					TextHandle->CreatePropertyValueWidget()
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0, 0.0))
				.AutoWidth()
				[
					ProjectHandle->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(2.0, 0.0))
				.MinWidth(80.f)
				[
					ProjectHandle->CreatePropertyValueWidget()
				]
			];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{

	}
};
