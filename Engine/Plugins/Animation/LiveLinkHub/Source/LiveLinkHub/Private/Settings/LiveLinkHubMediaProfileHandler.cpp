// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/LiveLinkHubMediaProfileHandler.h"

#include "Editor.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/SystemTimeTimecodeProvider.h"
#include "Engine/TimecodeProvider.h"
#include "MediaOutput.h"
#include "MediaSource.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"
#include "Settings/LiveLinkHubSessionExtraData_MediaProfile.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TimecodeCustomTimeStep.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHubMediaProfile, Log, All);

FLiveLinkHubMediaProfileHandler::FLiveLinkHubMediaProfileHandler()
{
	RegisterExtraDataHandler();

	// Listen for profile changes so we can adopt transient profiles created by other systems
	// (e.g. MediaProfileMenus) and ensure they have default timing instances.
	MediaProfileChangedHandle = IMediaProfileManager::Get().OnMediaProfileChanged().AddRaw(
		this, &FLiveLinkHubMediaProfileHandler::OnMediaProfileChanged);

	// Create and apply a default transient profile immediately so that TC/genlock
	// are registered with GEngine even if the user never opens the Media Profile tab.
	// OnExtraDataSessionLoaded will replace this with saved state when a config is loaded.
	UMediaProfile* DefaultProfile = CreateTransientMediaProfile();
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(DefaultProfile->NumMediaSources(), false);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(DefaultProfile->NumMediaOutputs(), false);
	IMediaProfileManager::Get().SetCurrentMediaProfile(DefaultProfile);
}

FLiveLinkHubMediaProfileHandler::~FLiveLinkHubMediaProfileHandler()
{
	if (MediaProfileChangedHandle.IsValid()
		&& FModuleManager::Get().IsModuleLoaded(TEXT("MediaProfile")))
	{
		IMediaProfileManager::Get().OnMediaProfileChanged().Remove(MediaProfileChangedHandle);
	}

	UnregisterExtraDataHandler();
}

TSubclassOf<ULiveLinkHubSessionExtraData> FLiveLinkHubMediaProfileHandler::GetExtraDataClass() const
{
	return ULiveLinkHubSessionExtraData_MediaProfile::StaticClass();
}

void FLiveLinkHubMediaProfileHandler::OnExtraDataSessionSaving(ULiveLinkHubSessionExtraData* InExtraData)
{
	ULiveLinkHubSessionExtraData_MediaProfile* MediaProfileData = CastChecked<ULiveLinkHubSessionExtraData_MediaProfile>(InExtraData);

	FLiveLinkHubMediaProfilePreset& Preset = MediaProfileData->MediaProfilePreset;

	// Clear previous preset state.
	Preset.MediaSources.Empty();
	Preset.MediaSourceLabels.Empty();
	Preset.MediaOutputs.Empty();
	Preset.MediaOutputLabels.Empty();
	Preset.TimecodeProvider = nullptr;
	Preset.CustomTimeStep = nullptr;
	Preset.bOverrideTimecodeProvider = false;
	Preset.bOverrideCustomTimeStep = false;

	UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!Profile)
	{
		return;
	}

	// Only snapshot sources/outputs that are OWNED by the media profile (Outer == Profile).
	// Sources injected by external systems (e.g. device streaming capabilities) have a different
	// Outer and are already persisted/restored by their own session extra data handlers.
	for (int32 i = 0; i < Profile->NumMediaSources(); ++i)
	{
		UMediaSource* Source = Profile->GetMediaSource(i);
		if (Source && Source->GetOuter() == Profile)
		{
			if (UMediaSource* DuplicatedSource = DuplicateObject<UMediaSource>(Source, InExtraData))
			{
				Preset.MediaSources.Add(DuplicatedSource);
				Preset.MediaSourceLabels.Add(Profile->GetLabelForMediaSource(i));
			}
			else
			{
				UE_LOG(LogLiveLinkHubMediaProfile, Warning, TEXT("Failed to duplicate media source at index %d for session save."), i);
			}
		}
	}

	// Snapshot media outputs owned by the profile.
	for (int32 i = 0; i < Profile->NumMediaOutputs(); ++i)
	{
		UMediaOutput* Output = Profile->GetMediaOutput(i);
		if (Output && Output->GetOuter() == Profile)
		{
			if (UMediaOutput* DuplicatedOutput = DuplicateObject<UMediaOutput>(Output, InExtraData))
			{
				Preset.MediaOutputs.Add(DuplicatedOutput);
				Preset.MediaOutputLabels.Add(Profile->GetLabelForMediaOutput(i));
			}
			else
			{
				UE_LOG(LogLiveLinkHubMediaProfile, Warning, TEXT("Failed to duplicate media output at index %d for session save."), i);
			}
		}
	}

	// Snapshot timecode provider.
	if (UTimecodeProvider* TC = Profile->GetTimecodeProvider())
	{
		Preset.TimecodeProvider = DuplicateObject<UTimecodeProvider>(TC, InExtraData);
	}

	// Snapshot custom time step.
	if (UEngineCustomTimeStep* CTS = Profile->GetCustomTimeStep())
	{
		Preset.CustomTimeStep = DuplicateObject<UEngineCustomTimeStep>(CTS, InExtraData);
	}

	// Access protected fields directly via friend class relationship.
	Preset.bOverrideTimecodeProvider = Profile->bOverrideTimecodeProvider;
	Preset.bOverrideCustomTimeStep = Profile->bOverrideCustomTimeStep;

	UE_LOG(LogLiveLinkHubMediaProfile, Log, TEXT("Saved media profile: %d sources, %d outputs, TC override=%d, CTS override=%d"),
		Preset.MediaSources.Num(), Preset.MediaOutputs.Num(),
		Preset.bOverrideTimecodeProvider, Preset.bOverrideCustomTimeStep);
}

void FLiveLinkHubMediaProfileHandler::OnExtraDataSessionLoaded(const ULiveLinkHubSessionExtraData* InExtraData)
{
	// Close any open editor for the current profile before swapping it out.
	// Track whether one was open so we can reopen after restore. This is persisted as a member
	// because ClearSession() and RestoreSession() trigger separate OnExtraDataSessionLoaded calls.
	if (UMediaProfile* OldProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
	{
		if (GEditor)
		{
			if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(OldProfile) > 0)
			{
				bEditorWasOpenBeforeClear = true;
			}
		}
	}

	// Reset the current profile.
	IMediaProfileManager::Get().SetCurrentMediaProfile(nullptr);
	TransientMediaProfile.Reset();

	if (!InExtraData)
	{
		// Recreate a default profile to maintain the invariant that one always exists.
		RestoreDefaultProfile();
		return;
	}

	const ULiveLinkHubSessionExtraData_MediaProfile* MediaProfileData =
		Cast<ULiveLinkHubSessionExtraData_MediaProfile>(InExtraData);

	if (!MediaProfileData)
	{
		RestoreDefaultProfile();
		return;
	}

	const FLiveLinkHubMediaProfilePreset& Preset = MediaProfileData->MediaProfilePreset;

	// Only restore if the preset has any meaningful content.
	if (Preset.MediaSources.Num() == 0 && Preset.MediaOutputs.Num() == 0
		&& !Preset.TimecodeProvider && !Preset.CustomTimeStep
		&& !Preset.bOverrideTimecodeProvider && !Preset.bOverrideCustomTimeStep)
	{
		RestoreDefaultProfile();
		return;
	}

	// Create a fresh transient profile.
	UMediaProfile* Profile = CreateTransientMediaProfile();

	// Restore only profile-owned sources. Device-injected sources will be re-added
	// by their own session restore handlers (e.g. LiveLinkDevice streaming capability).
	for (int32 i = 0; i < Preset.MediaSources.Num(); ++i)
	{
		UMediaSource* Source = Preset.MediaSources[i];
		if (Source)
		{
			if (UMediaSource* DuplicatedSource = DuplicateObject<UMediaSource>(Source, Profile))
			{
				Profile->AddMediaSource(DuplicatedSource);

				// Restore the user-defined label so the editor layout can match panels by name.
				if (Preset.MediaSourceLabels.IsValidIndex(i))
				{
					Profile->SetLabelForMediaSource(Profile->NumMediaSources() - 1, Preset.MediaSourceLabels[i]);
				}
			}
			else
			{
				UE_LOG(LogLiveLinkHubMediaProfile, Warning, TEXT("Failed to duplicate media source '%s' during session restore."), *Source->GetName());
			}
		}
	}

	// Restore profile-owned outputs.
	for (int32 i = 0; i < Preset.MediaOutputs.Num(); ++i)
	{
		UMediaOutput* Output = Preset.MediaOutputs[i];
		if (Output)
		{
			if (UMediaOutput* DuplicatedOutput = DuplicateObject<UMediaOutput>(Output, Profile))
			{
				Profile->AddMediaOutput(DuplicatedOutput);

				// Restore the user-defined label.
				if (Preset.MediaOutputLabels.IsValidIndex(i))
				{
					Profile->SetLabelForMediaOutput(Profile->NumMediaOutputs() - 1, Preset.MediaOutputLabels[i]);
				}
			}
			else
			{
				UE_LOG(LogLiveLinkHubMediaProfile, Warning, TEXT("Failed to duplicate media output '%s' during session restore."), *Output->GetName());
			}
		}
	}

	// Access protected fields directly via friend class relationship.
	if (Preset.TimecodeProvider)
	{
		Profile->TimecodeProvider = DuplicateObject<UTimecodeProvider>(Preset.TimecodeProvider, Profile);
	}
	Profile->bOverrideTimecodeProvider = Preset.bOverrideTimecodeProvider;

	if (Preset.CustomTimeStep)
	{
		Profile->CustomTimeStep = DuplicateObject<UEngineCustomTimeStep>(Preset.CustomTimeStep, Profile);
	}
	Profile->bOverrideCustomTimeStep = Preset.bOverrideCustomTimeStep;

	// Ensure default timing instances exist even if the preset didn't have them.
	EnsureDefaultTimingInstances(Profile);

	// Ensure the proxy arrays match the restored source/output counts BEFORE Apply() runs.
	// Apply() calls FixNumSourcesAndOutputs() which resizes MediaSources/MediaOutputs to match
	// the proxy count - if proxies aren't set up first, our restored arrays get wiped to zero.
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(Profile->NumMediaSources(), false);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(Profile->NumMediaOutputs(), false);

	// Set as the current media profile - this calls Apply().
	IMediaProfileManager::Get().SetCurrentMediaProfile(Profile);

	UE_LOG(LogLiveLinkHubMediaProfile, Log, TEXT("Restored media profile: %d sources, %d outputs"),
		Profile->NumMediaSources(), Profile->NumMediaOutputs());

	// If an editor was open for the previous profile (closed during clear), reopen for the new one.
	if (bEditorWasOpenBeforeClear && GEditor)
	{
		bEditorWasOpenBeforeClear = false;
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Profile);
	}
}

void FLiveLinkHubMediaProfileHandler::RestoreDefaultProfile()
{
	// No saved profile to restore. Recreate a fresh default so that a transient profile
	// always exists and TC/genlock settings remain applied.
	UMediaProfile* DefaultProfile = CreateTransientMediaProfile();
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(DefaultProfile->NumMediaSources(), false);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(DefaultProfile->NumMediaOutputs(), false);
	IMediaProfileManager::Get().SetCurrentMediaProfile(DefaultProfile);
}

void FLiveLinkHubMediaProfileHandler::OnMediaProfileChanged(UMediaProfile* PreviousProfile, UMediaProfile* NewProfile)
{
	if (!NewProfile)
	{
		return;
	}

	// If a transient profile was created externally (e.g. by MediaProfileMenus), adopt it
	// so that we track it and can add default timing instances.
	if (NewProfile->GetPackage() == GetTransientPackage() && NewProfile != TransientMediaProfile.Get())
	{
		TransientMediaProfile = TStrongObjectPtr<UMediaProfile>(NewProfile);
		EnsureDefaultTimingInstances(NewProfile);
	}
}

void FLiveLinkHubMediaProfileHandler::EnsureDefaultTimingInstances(UMediaProfile* Profile)
{
	if (!Profile)
	{
		return;
	}

	// Populate default timecode provider and custom time step with overrides enabled
	// so that Apply() registers them with GEngine. This ensures TC/genlock are active
	// on startup without requiring the user to open the Media Profile editor first.
	if (!Profile->TimecodeProvider)
	{
		Profile->TimecodeProvider = NewObject<USystemTimeTimecodeProvider>(Profile);
		Profile->bOverrideTimecodeProvider = true;
	}

	if (!Profile->CustomTimeStep)
	{
		Profile->CustomTimeStep = NewObject<UTimecodeCustomTimeStep>(Profile);
		Profile->bOverrideCustomTimeStep = true;
	}
}

UMediaProfile* FLiveLinkHubMediaProfileHandler::CreateTransientMediaProfile()
{
	TransientMediaProfile.Reset();
	
	UMediaProfile* Profile = NewObject<UMediaProfile>(GetTransientPackage());

	EnsureDefaultTimingInstances(Profile);

	TransientMediaProfile = TStrongObjectPtr<UMediaProfile>(Profile);
	return Profile;
}
