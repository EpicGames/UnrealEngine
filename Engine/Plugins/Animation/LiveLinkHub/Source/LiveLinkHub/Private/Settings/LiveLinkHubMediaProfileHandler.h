// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"

class UMediaProfile;

/**
 * Manages the transient UMediaProfile lifecycle for Live Link Hub.
 *
 * Transient profiles can be created by two paths:
 * - Session restore (OnExtraDataSessionLoaded) creates one from saved preset data.
 * - MediaProfileMenus creates one when the user clicks the toolbar button.
 *
 * This handler adopts any transient profile set as current via OnMediaProfileChanged,
 * tracking it in a TStrongObjectPtr to prevent GC and ensuring default timecode/genlock
 * instances are populated. Session save/load is handled via the ExtraData system.
 *
 * NOTE: This handler has an implicit ordering dependency with other ExtraData handlers.
 * During session restore, this handler creates the transient UMediaProfile and sets it as
 * the current profile via IMediaProfileManager. Other handlers (e.g. LiveLinkDevice streaming)
 * may inject their own media sources into the active profile during or after their own restore.
 * The current design is order-independent because device sources are injected via the device
 * subsystem's own initialization path, not during OnExtraDataSessionLoaded. If that changes,
 * handler ordering (determined by modular feature registration order) may need to be explicit.
 */
class FLiveLinkHubMediaProfileHandler : public ILiveLinkHubSessionExtraDataHandler
{
public:
	FLiveLinkHubMediaProfileHandler();
	virtual ~FLiveLinkHubMediaProfileHandler();

	//~ Begin ILiveLinkHubSessionExtraDataHandler interface
	virtual TSubclassOf<ULiveLinkHubSessionExtraData> GetExtraDataClass() const override;
	virtual void OnExtraDataSessionSaving(ULiveLinkHubSessionExtraData* InExtraData) override;
	virtual void OnExtraDataSessionLoaded(const ULiveLinkHubSessionExtraData* InExtraData) override;
	//~ End ILiveLinkHubSessionExtraDataHandler interface

private:
	/** Create a fresh transient UMediaProfile in the transient package with default TC/CTS. */
	UMediaProfile* CreateTransientMediaProfile();

	/** Populates default timecode provider and custom time step on a profile if they are null. */
	void EnsureDefaultTimingInstances(UMediaProfile* Profile);

	/** Recreate a fresh default transient profile when no saved state is available. */
	void RestoreDefaultProfile();

	/** Called when the active MediaProfile changes. Adopts externally-created transient profiles. */
	void OnMediaProfileChanged(UMediaProfile* PreviousProfile, UMediaProfile* NewProfile);

	/** The transient media profile managed by this handler. Kept alive via TStrongObjectPtr. */
	TStrongObjectPtr<UMediaProfile> TransientMediaProfile;

	/** Tracks whether the editor was open when the profile was cleared, so we can reopen on restore. */
	bool bEditorWasOpenBeforeClear = false;

	/** Handle for the OnMediaProfileChanged delegate. */
	FDelegateHandle MediaProfileChangedHandle;
};
