// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkHubSessionExtraData_MediaProfile.generated.h"

class UEngineCustomTimeStep;
class UMediaOutput;
class UMediaSource;
class UTimecodeProvider;

/**
 * Preset struct that captures the state of a UMediaProfile for session serialization.
 * FJsonObjectConverter handles the Instanced polymorphic subobjects automatically.
 */
USTRUCT()
struct FLiveLinkHubMediaProfilePreset
{
	GENERATED_BODY()

	/** Media sources configured in the profile. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMediaSource>> MediaSources;

	/** User-defined labels for media sources (parallel array to MediaSources). */
	UPROPERTY()
	TArray<FString> MediaSourceLabels;

	/** Media outputs configured in the profile. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMediaOutput>> MediaOutputs;

	/** User-defined labels for media outputs (parallel array to MediaOutputs). */
	UPROPERTY()
	TArray<FString> MediaOutputLabels;

	/** Whether the profile overrides the engine's timecode provider. */
	UPROPERTY()
	bool bOverrideTimecodeProvider = false;

	/** Timecode provider instance (only relevant if bOverrideTimecodeProvider is true). */
	UPROPERTY(Instanced)
	TObjectPtr<UTimecodeProvider> TimecodeProvider;

	/** Whether the profile overrides the engine's custom time step. */
	UPROPERTY()
	bool bOverrideCustomTimeStep = false;

	/** Custom time step instance (only relevant if bOverrideCustomTimeStep is true). */
	UPROPERTY(Instanced)
	TObjectPtr<UEngineCustomTimeStep> CustomTimeStep;
};

/**
 * Session extra data for persisting the transient MediaProfile state in LLH config files.
 * Follows the same pattern as ULiveLinkHubSessionExtraData_Device.
 */
UCLASS()
class ULiveLinkHubSessionExtraData_MediaProfile : public ULiveLinkHubSessionExtraData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FLiveLinkHubMediaProfilePreset MediaProfilePreset;
};
