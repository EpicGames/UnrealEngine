// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubSettings.h"

#include "Algo/Transform.h"
#include "Algo/Reverse.h"
#include "Config/LiveLinkHubTemplateTokens.h"
#include "Engine/Engine.h"
#include "NamingTokensEngineSubsystem.h"
#include "Session/LiveLinkHubSession.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkHubSettings)

void ULiveLinkHubSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSettings, FilenameTemplate))
	{
		CalculateExampleOutput();
	}
}

ULiveLinkHubSettings::ULiveLinkHubSettings()
{
}

void ULiveLinkHubSettings::PostInitProperties()
{
	Super::PostInitProperties();
}

void ULiveLinkHubSettings::CalculateExampleOutput()
{
	if (const TObjectPtr<ULiveLinkHubNamingTokens> Tokens = GetNamingTokens())
	{
		FNamingTokenFilterArgs Filter;
		Filter.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
		
		check(GEngine);
		const FNamingTokenResultData TemplateData = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->EvaluateTokenString(
			GetDefault<ULiveLinkHubSettings>()->FilenameTemplate, Filter);
		FilenameOutput = TemplateData.EvaluatedText.ToString();
	}
	else
	{
		FilenameOutput = GetDefault<ULiveLinkHubSettings>()->FilenameTemplate;
	}
}

TObjectPtr<ULiveLinkHubNamingTokens> ULiveLinkHubSettings::GetNamingTokens() const
{
	if (NamingTokens == nullptr)
	{
		NamingTokens = NewObject<ULiveLinkHubNamingTokens>(const_cast<ULiveLinkHubSettings*>(this),
			ULiveLinkHubNamingTokens::StaticClass());
		NamingTokens->CreateDefaultTokens();
	}

	return NamingTokens;
}

void ULiveLinkHubUserSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Cache the filter data on load.
	UE::TWriteScopeLock Lock{ FiltersLock };
	CurrentFilterPreset_ThreadSafe = CurrentPreset;
}

void ULiveLinkHubUserSettings::CacheRecentConfig(const FString& SavePath)
{
	LastConfigDirectory = FPaths::GetPath(SavePath);

	// Make sure this ends up at the "top" if we touch this file.
	RecentConfigs.Remove(SavePath);
	RecentConfigs.Add(SavePath);

	if (RecentConfigs.Num() >= MaxNumberOfRecentConfigs && MaxNumberOfRecentConfigs > 0)
	{
		RecentConfigs.RemoveAt(0, EAllowShrinking::No);
	}

	SaveConfig();
}

TArray<FLiveLinkHubSessionFile> ULiveLinkHubUserSettings::GetRecentConfigFiles()
{
	// Clear entries that no longer exist.
	const bool bRemovedElement = RecentConfigs.RemoveAll([](const FString& FilePath) { return !FPaths::FileExists(FilePath); }) > 0;
	if (bRemovedElement)
	{
		SaveConfig();
	}

	TArray<FLiveLinkHubSessionFile> Files;
	Algo::Transform(RecentConfigs, Files, UE_PROJECTION(FLiveLinkHubSessionFile));
	Algo::Reverse(Files);

	return Files;
}

void ULiveLinkHubUserSettings::SaveCurrentPreset()
{
	for (FLiveLinkHubClientFilterPreset& Preset : FilterPresets)
	{
		if (Preset.PresetName == CurrentPreset.PresetName)
		{
			Preset = CurrentPreset;
			break;
		}
	}

	SaveConfig();
}

void ULiveLinkHubUserSettings::SavePresetAs(const FString& PresetName)
{
	FilterPresets.Add_GetRef(CurrentPreset).PresetName = PresetName;
	CurrentPreset.PresetName = PresetName;

	{
		UE::TWriteScopeLock Lock{ FiltersLock };
		CurrentFilterPreset_ThreadSafe = CurrentPreset;
	}

	SaveConfig();
}

void ULiveLinkHubUserSettings::SaveClientsToPreset(const FString& InPresetName)
{
	TArray<TPair<FString, FString>> IPAndProject;
	if (ILiveLinkHubClientsModel* ClientsModel = ILiveLinkHubClientsModel::Get())
	{
		for (const FLiveLinkHubClientId& ClientId : ClientsModel->GetSessionClients())
		{
			if (TOptional<FLiveLinkHubUEClientInfo> Info = ClientsModel->GetClientInfo(ClientId))
			{
				if (!Info->IPAddress.IsEmpty())
				{
					IPAndProject.Emplace(Info->IPAddress, Info->ProjectName);
				}
			}
		}
	}

	FLiveLinkHubClientFilterPreset Preset;
	Preset.PresetName = InPresetName;
	Preset.Filters.Reserve(IPAndProject.Num());
	Preset.AutoConnectClients = CurrentPreset.AutoConnectClients;

	Algo::Transform(IPAndProject, Preset.Filters, [](const TPair<FString, FString>& IpAndProject)
		{
			return FLiveLinkHubClientTextFilter{ IpAndProject.Value, IpAndProject.Key, ELiveLinkHubClientFilterType::IP, ELiveLinkHubClientFilterBehavior::Include };
		});

	int32 Index = FilterPresets.IndexOfByPredicate([&InPresetName](const FLiveLinkHubClientFilterPreset& Preset) { return InPresetName == Preset.PresetName; });
	if (Index != INDEX_NONE)
	{
		FilterPresets[Index] = Preset;
	}
	else
	{
		FilterPresets.Add(Preset);
	}

	CurrentPreset = Preset;

	SaveConfig();

	UE::TWriteScopeLock Lock{ FiltersLock };
	CurrentFilterPreset_ThreadSafe = CurrentPreset;
}

void ULiveLinkHubUserSettings::LoadFilterPreset(const FString& PresetName)
{
	if (FLiveLinkHubClientFilterPreset* Preset = Algo::FindBy(FilterPresets, PresetName, &FLiveLinkHubClientFilterPreset::PresetName))
	{
		{
			UE::TWriteScopeLock Lock{ FiltersLock };
			CurrentPreset = *Preset;
		}

		PostUpdateClientFilters();
	}
	else
	{
		UE_LOGF(LogLiveLinkHub, Warning, "Could not find preset %ls.", *PresetName);
	}
}

void ULiveLinkHubUserSettings::DeleteFilterPreset(const FString& PresetName)
{
	FilterPresets.RemoveAll([&PresetName](const FLiveLinkHubClientFilterPreset& InPreset) { return PresetName == InPreset.PresetName; });
	if (PresetName == CurrentPreset.PresetName)
	{
		CurrentPreset = FLiveLinkHubClientFilterPreset{};

		UE::TWriteScopeLock Lock{ FiltersLock };
		CurrentFilterPreset_ThreadSafe = CurrentPreset;
	}
	SaveConfig();
}
