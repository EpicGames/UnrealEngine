// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Clients/LiveLinkHubUEClientInfo.h"
#include "Config/LiveLinkHubTemplateTokens.h"
#include "UObject/Object.h"
#include "ILiveLinkHubClientsModel.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubMessages.h"
#include "Misc/FrameRate.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkHubSettings.generated.h"

#define UE_API LIVELINKHUB_API

/**
 * Settings for LiveLinkHub.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class ULiveLinkHubSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULiveLinkHubSettings();
	
	UE_API virtual void PostInitProperties() override;
	
	/** Parse templates and set example output fields. */
	UE_API void CalculateExampleOutput();

	/** Get the naming tokens for Live Link Hub. */
	UE_API TObjectPtr<ULiveLinkHubNamingTokens> GetNamingTokens() const;
	
protected:
	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
	
public:
	/** Config to apply when starting LiveLinkHub. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	FFilePath StartupConfig;

	/** The size in megabytes to buffer when streaming a recording. */
	UPROPERTY(config, EditAnywhere, Category="LiveLinkHub", meta = (ClampMin = "1", UIMin = "1"))
	int32 PlaybackFrameBufferSizeMB = 100;

	/** Number of frames to buffer at once. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLinkHub", meta = (ClampMin = "2", UIMin = "2"))
	int32 PlaybackBufferBatchSize = 5;

	/** Maximum number of frame ranges to store in history while scrubbing. Increasing can make scrubbing faster but temporarily use more memory. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLinkHub")
	int32 PlaybackMaxBufferRangeHistory = 25;
	
	UPROPERTY(EditAnywhere, Config, Category = "LiveLinkHub")
	bool bAllowModifyingSourceSettingsInPlayback = false;

	/** Which project settings sections to display when opening the settings viewer. */
	UPROPERTY(config)
	TArray<FName> ProjectSettingsToDisplay;

	/** If this is enabled, invalid subjects will be removed after loading a session. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bRemoveInvalidSubjectsAfterLoadingSession = false;

	/** If enabled, you'll be asked to confirm if you really want to close the hub. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bConfirmClose = true;

	/** Whether to enable Live Link Hub's crash recovery system. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bEnableCrashRecovery = true;

	/** Whether to show the app's frame rate in the top right corner. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bShowFrameRate = false;

	/** Whether to show memory usage in the top right corner. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bShowMemoryUsage = true;

	/** How much RAM (in MB) the program can use before showing a warning. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub", DisplayName = "Memory Warning Threshold (MB)")
	float ShowMemoryWarningThresholdMB = 8000.0;

	/**
	 * - Experimental - If this is disabled, LiveLinkHub's LiveLink Client will tick outside of the game thread.
	 * This allows processing LiveLink frame snapshots without the risk of being blocked by the game / ui thread.
	 * Note that this should only be relevant for virtual subjects since data is already forwarded to UE outside of the game thread.
	 */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub", meta = (ConfigRestartRequired = true))
	bool bTickOnGameThread = false;

	/** Target framerate for ticking LiveLinkHub. */
	UPROPERTY(config, EditAnywhere, Category="LiveLinkHub", meta = (ConfigRestartRequired = true, ClampMin="15.0"))
	float TargetFrameRate = 60.0f;

	/** Whether to prompt the user to pick a save directory after doing a recording. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bPromptSaveAsOnRecord = false;

	/** Whether to increment take number when stopping a recording. */
	UPROPERTY(config, EditAnywhere, Category = "Templates")
	bool bIncrementTakeOnRecordingEnd = true;

	/** Whether to enable auto-save of Live Link Hub configs. */
	UPROPERTY(config, EditAnywhere, Category = "Autosave")
	bool bEnableAutosave = true;

	/** How often should we trigger a config autosave.*/
	UPROPERTY(config, EditAnywhere, Category = "Autosave", meta=(ClampMin="1", ClampMax="120", EditCondition="bEnableAutosave", EditConditionHides))
	uint32 MinutesBetweenAutosave = 5;

	/** How many autosave file should we retain on disk. */
	UPROPERTY(config, EditAnywhere, Category = "Autosave", meta = (ClampMin = "1", ClampMax = "10", EditCondition = "bEnableAutosave", EditConditionHides))
	int32 NumberOfAutosaveFilesToRetain = 5;

	/** Maximum time in seconds to wait for sources to clean up. Increase this value if you notice that some sources are incorrectly cleaned up when switching a config. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLinkHub", meta = (ClampMin="0.0"))
	float SourceMaxCleanupTime = 0.25f;
	
	/** The filename template to use when creating recordings. */
	UPROPERTY(config, EditAnywhere, Category="Templates")
	FString FilenameTemplate = TEXT("{session}_{slate}_tk{take}");

	/** Example parsed output of the template. */
	UPROPERTY(VisibleAnywhere, Category="Templates", DisplayName="Output")
	FString FilenameOutput;
	
	/** Placeholder for a list of the automatic tokens, set from the customization. */
	UPROPERTY(VisibleAnywhere, Category="Templates")
	FText AutomaticTokens;

private:
	/**
	 * Naming tokens for Live Link, instantiated each load based on the naming tokens class.
	 * This isn't serialized to the config file, and exists here for singleton-like access.
	 */
	UPROPERTY(Instanced, Transient)
	mutable TObjectPtr<ULiveLinkHubNamingTokens> NamingTokens;
};

/** Whether a filter should include or exlude the client that matches the filter. */
UENUM()
enum class ELiveLinkHubClientFilterBehavior
{
	Include,
	Exclude
};

/** Whether we're filtering on IP or a hostname. */
UENUM()
enum class ELiveLinkHubClientFilterType
{
	IP,
	Host
};

/** Basic text filter that includes or excludes IPs/Hostnames. */
USTRUCT()
struct FLiveLinkHubClientTextFilter
{
	GENERATED_BODY()

	/** Optional project name filter. Will match any project if empty or *. */
	UPROPERTY(config, EditAnywhere, Category = "Filters")
	FString Project;

	/** Text that will be used to match the filter. */
	UPROPERTY(config, EditAnywhere, Category = "Filters")
	FString Text;

	/** Type of the filter. */
	UPROPERTY(config, EditAnywhere, Category = "Filters")
	ELiveLinkHubClientFilterType Type = ELiveLinkHubClientFilterType::IP;

	/** Behavior of the filter. */
	UPROPERTY(config, EditAnywhere, Category = "Filters")
	ELiveLinkHubClientFilterBehavior Behavior = ELiveLinkHubClientFilterBehavior::Include;
};

/** A collection of filters, used to restore and save filters on disk. */
USTRUCT()
struct FLiveLinkHubClientFilterPreset
{
	GENERATED_BODY()

	/** Name of this preset. */
	UPROPERTY(config, DisplayName="Name")
	FString PresetName;

	/** General autoconnect mode of the app. */
	UPROPERTY(config)
	ELiveLinkHubAutoConnectMode AutoConnectClients = ELiveLinkHubAutoConnectMode::LocalOnly;

	/** List of filters in the preset. */
	UPROPERTY(config, EditAnywhere, Category = "Filters")
	TArray<FLiveLinkHubClientTextFilter> Filters;
};


/**
 * User Settings for LiveLinkHub.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class ULiveLinkHubUserSettings : public UObject
{
	GENERATED_BODY()

public:
	//~ UObject Interface
	virtual void PostInitProperties() override;

	/** Number of recent configs to keep track. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	uint8 MaxNumberOfRecentConfigs = 5;

	/** File paths to recent configs that were saved/loaded. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	TArray<FString> RecentConfigs;

	/** Which directories to scan to discover layouts */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	TArray<FString> LayoutDirectories;

	/** The last directory of the a config that was saved or loaded. */
	UPROPERTY(config)
	FString LastConfigDirectory;

	UPROPERTY(config, EditAnywhere, Category = "Filters", meta=(ShowOnlyInnerProperties))
	FLiveLinkHubClientFilterPreset CurrentPreset;

public:
	/** Call after an explicit config save or load to save it in the recent config file list. */
	void CacheRecentConfig(const FString& SavePath);

	/** Get the most recent config / session files that were used in LiveLinkHub. */
	TArray<struct FLiveLinkHubSessionFile> GetRecentConfigFiles();

	/** Invoked through post-edit change, but can be invoked manually to copy the filters data to a thread-safe copy.*/
	void PostUpdateClientFilters()
	{
		{
			UE::TWriteScopeLock Lock{ FiltersLock };
			CurrentFilterPreset_ThreadSafe = CurrentPreset;
		}

		OnFiltersModifiedDelegate.Broadcast();

		SaveConfig();
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
	{
		PostUpdateClientFilters();
	}
#endif

	/** Get the AutoConnect mode from the current preset. */
	ELiveLinkHubAutoConnectMode GetAutoConnectMode() const 
	{
		UE::TReadScopeLock Lock{ FiltersLock };
		return CurrentFilterPreset_ThreadSafe.AutoConnectClients;
	}

	/** Sets the autoconnect mode. */
	void SetAutoConnectMode(ELiveLinkHubAutoConnectMode Mode)
	{
		CurrentPreset.AutoConnectClients = Mode;

		PostUpdateClientFilters();
	}

	/** Save the current preset to the settings ini. */
	void SaveCurrentPreset();

	/** Save the current filters configuration as a new preset in the settings ini. */
	void SavePresetAs(const FString& PresetName);

	/** Creates a preset of filters from the list of clients that are currently in the session. */
	void SaveClientsToPreset(const FString& InPresetName);

	/** Get all filter presets. */
	const TArray<FLiveLinkHubClientFilterPreset>& GetFilterPresets() const
	{
		return FilterPresets;
	}

	/** Applies a filter preset. */
	void LoadFilterPreset(const FString& PresetName);

	/** Deletes a filter preset. */
	void DeleteFilterPreset(const FString& PresetName);

	/** Get a copy of the client filters data in a thread-safe way. */
	FLiveLinkHubClientFilterPreset GetClientFiltersData_AnyThread()
	{
		UE::TReadScopeLock Lock{ FiltersLock };
		return CurrentFilterPreset_ThreadSafe;
	}

	DECLARE_TS_MULTICAST_DELEGATE(FOnFiltersModified);
	FOnFiltersModified OnFiltersModifiedDelegate;

private:
	/** 
	 * Shadow variable needed to access the current filter preset from outside the game thread. 
	 * Used to avoid excessive locking from widgets ticking on the game thread and accessing CurrentPreset. 
	 */
	FLiveLinkHubClientFilterPreset CurrentFilterPreset_ThreadSafe;

	/** List of filter presets serialized to the user config. */
	UPROPERTY(config)
	TArray<FLiveLinkHubClientFilterPreset> FilterPresets;

	/** Lock used to access CurrentFilterPreset_ThreadSafe. */
	mutable FTransactionallySafeRWLock FiltersLock;
};

#undef UE_API

