// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubSettingsCustomization.h"

#include "Config/LiveLinkHubTemplateTokens.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Engine.h"
#include "LiveLinkHubSettings.h"
#include "NamingTokensEngineSubsystem.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubSettingsCustomization"

TSharedRef<IDetailCustomization> FLiveLinkHubSettingsCustomization::MakeInstance()
{
	return MakeShared<FLiveLinkHubSettingsCustomization>();
}

void FLiveLinkHubSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Update current value when opening the settings page. Not safe to do under the settings object since there isn't
	// an explicit callback when the settings page is opened, and using a method like PostInitProperties fires on the CDO
	// early in the startup process.
	GetMutableDefault<ULiveLinkHubSettings>()->CalculateExampleOutput();
	
	const TSharedRef<IPropertyHandle> AutomaticTokensHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkHubSettings, AutomaticTokens));
	IDetailPropertyRow* AutomaticTokensRow = DetailBuilder.EditDefaultProperty(AutomaticTokensHandle);
	check(AutomaticTokensRow);
	
	AutomaticTokensRow->CustomWidget()
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.f, 4.f)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(AutomaticTokensHandle->GetPropertyDisplayName())
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SEditableText)
			.IsReadOnly(true)
			.Text_Raw(this, &FLiveLinkHubSettingsCustomization::GetDisplayTokenText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

FText FLiveLinkHubSettingsCustomization::GetDisplayTokenText() const
{
	FString FormattedTokensString = TEXT("None");
	if (GEngine)
	{
		FNamingTokenFilterArgs Args;
		if (const TObjectPtr<ULiveLinkHubNamingTokens> Tokens = GetDefault<ULiveLinkHubSettings>()->GetNamingTokens())
		{
			Args.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
		}
		FormattedTokensString = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->GetFormattedTokensStringForDisplay(MoveTemp(Args));
	}
	return FText::FromString(FormattedTokensString);
}

#undef LOCTEXT_NAMESPACE