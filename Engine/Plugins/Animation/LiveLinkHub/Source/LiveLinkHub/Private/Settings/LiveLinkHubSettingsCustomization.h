// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

/**
 * Customization for the live link hub settings.
 */
class FLiveLinkHubSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	FText GetDisplayTokenText() const;
};