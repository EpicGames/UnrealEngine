// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Clients/LiveLinkHubProvider.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkSourceSettings.h"
#include "Settings/LiveLinkHubTimingUtils.h"

#include "LiveLinkHubTimeAndSyncSettings.generated.h"

/**
 * Settings for LiveLinkHub's timecode and genlock broadcasting.
 *
 * Timecode and genlock source configuration is defined in the active MediaProfile.
 * This class controls whether LiveLinkHub broadcasts those settings to connected clients.
 */
UCLASS(config=EditorPerProjectUserSettings)
class ULiveLinkHubTimeAndSyncSettings : public UObject
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.8,"Property will be removed in a future version")
	UPROPERTY(config, meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	FLiveLinkHubTimecodeSettings TimecodeSettings;
	
	/** Whether the hub should be used as a timecode source for connected clients. */
	UPROPERTY(config, EditAnywhere, Category = "Timecode", DisplayName = "Enable")
	bool bUseLiveLinkHubAsTimecodeSource = false;
	
	UE_DEPRECATED(5.8,"Property will be removed in a future version")
	/** Custom time step */
	UPROPERTY(config, meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	FLiveLinkHubCustomTimeStepSettings CustomTimeStepSettings;

	/** Whether the hub should be used as a CustomTimeStep source for connected clients. */
	UPROPERTY(config, EditAnywhere, Category = "Frame Lock", DisplayName = "Enable")
	bool bUseLiveLinkHubAsCustomTimeStepSource = false;

	/**
	 * Evaluation mode that connected Editor clients should apply to their Live Link Hub source.
	 * Pushed to every connected client when changed and when a client connects. Overwrites the client's local setting.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Evaluation", DisplayName = "Preferred Client Evaluation Mode")
	ELiveLinkSourceMode PreferredClientEvaluationMode = ELiveLinkSourceMode::EngineTime;

public:
#if WITH_EDITOR
	//~ Begin UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		const FName PropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

		if (PropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkHubTimeAndSyncSettings, bUseLiveLinkHubAsTimecodeSource))
		{
			OnToggleTimecodeSettings();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkHubTimeAndSyncSettings, bUseLiveLinkHubAsCustomTimeStepSource))
		{
			OnToggleCustomTimeStepSettings();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkHubTimeAndSyncSettings, PreferredClientEvaluationMode))
		{
			if (const TSharedPtr<FLiveLinkHub> Hub = FLiveLinkHub::Get())
			{
				if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = Hub->GetLiveLinkProvider())
				{
					LiveLinkProvider->UpdateSourceEvaluationMode(PreferredClientEvaluationMode);
				}
			}
		}
	}
	//~ End UObject interface
#endif

	/** Handles broadcasting timecode settings when they're enabled/disabled. */
	void OnToggleTimecodeSettings()
	{
		if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider())
		{
			if (bUseLiveLinkHubAsTimecodeSource)
			{
				// MediaProfile handles local engine application; we just broadcast to clients.
				LiveLinkProvider->UpdateTimecodeSettings(UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile());
			}
			else
			{
				LiveLinkProvider->ResetTimecodeSettings();
			}
		}
	}

	/** Handles broadcasting custom time step settings when they're enabled/disabled. */
	void OnToggleCustomTimeStepSettings()
	{
		if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider())
		{
			if (bUseLiveLinkHubAsCustomTimeStepSource)
			{
				// MediaProfile handles local engine application; we just broadcast to clients.
				LiveLinkProvider->UpdateCustomTimeStepSettings(UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile());
			}
			else
			{
				LiveLinkProvider->ResetCustomTimeStepSettings();
			}
		}
	}
};
