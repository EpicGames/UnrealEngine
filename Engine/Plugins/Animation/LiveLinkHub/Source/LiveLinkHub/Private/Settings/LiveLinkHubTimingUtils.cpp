// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/LiveLinkHubTimingUtils.h"

#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "LiveLinkHub.h"
#include "LiveLinkHubLog.h"
#include "LiveLinkHubTimeAndSyncSettings.h"
#include "Clients/LiveLinkHubProvider.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"

TOptional<ELiveLinkTimecodeProviderEvaluationType> UE::LiveLinkHubTimingUtils::Private::PlaybackEvaluationOverride;

FLiveLinkHubTimecodeSettings UE::LiveLinkHubTimingUtils::Private::GetTimecodeSettingsFromActiveProfile()
{
	UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!Profile)
	{
		return FLiveLinkHubTimecodeSettings();
	}

	FLiveLinkHubTimecodeSettings Settings = FLiveLinkHubTimecodeSettings::FromTimecodeProvider(Profile->GetTimecodeProvider());

	if (PlaybackEvaluationOverride.IsSet())
	{
		Settings.OverrideEvaluationType = PlaybackEvaluationOverride;
	}

	return Settings;
}

FLiveLinkHubCustomTimeStepSettings UE::LiveLinkHubTimingUtils::Private::GetCustomTimeStepSettingsFromActiveProfile()
{
	UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!Profile)
	{
		FLiveLinkHubCustomTimeStepSettings Settings;
		Settings.Kind = ELiveLinkHubCustomTimeStepKind::Reset;
		Settings.bResetCustomTimeStep = true;
		return Settings;
	}

	return FLiveLinkHubCustomTimeStepSettings::FromCustomTimeStep(Profile->GetCustomTimeStep());
}


void UE::LiveLinkHubTimingUtils::Private::ClearMediaProfileCustomTimeStepFromEngine()
{
	// Hub broadcasts the MediaProfile's custom time step to clients but should never use a custom timestep itself. 
	// UMediaProfile::Apply() unconditionally pushes its custom time step to GEngine, so we undo that push here. 
	// We only clear the engine's current custom time step if it matches the active MediaProfile's. A custom time step 
	// coming from any other path should still succeed.
	if (!GEngine)
	{
		return;
	}

	UMediaProfile* ActiveProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveProfile)
	{
		return;
	}

	UEngineCustomTimeStep* ProfileCustomTimeStep = ActiveProfile->GetCustomTimeStep();
	if (ProfileCustomTimeStep && GEngine->GetCustomTimeStep() == ProfileCustomTimeStep)
	{
		UE_LOGF(LogLiveLinkHub, Display,
			"Live Link Hub does not run as a genlock target — clearing MediaProfile custom time step '%ls' from GEngine.",
			*ProfileCustomTimeStep->GetName());
		GEngine->SetCustomTimeStep(nullptr);
	}
}

void UE::LiveLinkHubTimingUtils::Private::BroadcastCurrentSettings()
{
	if (FLiveLinkHub::Get())
	{
		if (const TSharedPtr<FLiveLinkHubProvider> LiveLinkProvider = FLiveLinkHub::Get()->GetLiveLinkProvider())
		{
			if (const ULiveLinkHubTimeAndSyncSettings* Settings = GetDefault<ULiveLinkHubTimeAndSyncSettings>())
			{
				if (Settings->bUseLiveLinkHubAsTimecodeSource)
				{
					LiveLinkProvider->UpdateTimecodeSettings(GetTimecodeSettingsFromActiveProfile());
				}
				if (Settings->bUseLiveLinkHubAsCustomTimeStepSource)
				{
					LiveLinkProvider->UpdateCustomTimeStepSettings(GetCustomTimeStepSettingsFromActiveProfile());
				}
			}
		}
	}

	ClearMediaProfileCustomTimeStepFromEngine();
}
