// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubMessages.h"

class UMediaProfile;

/**
 * Utility functions for reading timecode/genlock settings from the active MediaProfile
 * and converting them to LiveLinkHub message format for broadcasting to clients.
 */
namespace UE::LiveLinkHubTimingUtils::Private
{
	/** Read the active MediaProfile's timecode provider and convert to message settings. Returns NotDefined if no profile is active. */
	FLiveLinkHubTimecodeSettings GetTimecodeSettingsFromActiveProfile();

	/** Read the active MediaProfile's custom time step and convert to message settings. Returns reset settings if no profile is active. */
	FLiveLinkHubCustomTimeStepSettings GetCustomTimeStepSettingsFromActiveProfile();
	
	/** Broadcasts current timecode settings to all connected clients if broadcasting is enabled. Called when the MediaProfile changes. */
	void BroadcastCurrentSettings();

	/**
	 * Clear GEngine's custom time step if it matches the one defined in the active MediaProfile.
	 */
	void ClearMediaProfileCustomTimeStepFromEngine();

	/**
	 * Transient override for the timecode evaluation type during playback.
	 * When set, this overrides the EvaluationType in the settings returned by GetTimecodeSettingsFromActiveProfile().
	 */
	extern TOptional<ELiveLinkTimecodeProviderEvaluationType> PlaybackEvaluationOverride;
}
