// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSettingsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "LiveLinkSettings.h"

TSharedRef<IDetailCustomization> FLiveLinkSettingsCustomization::MakeInstance()
{
	return MakeShared<FLiveLinkSettingsCustomization>();
}

void FLiveLinkSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// These properties aren't needed when accessed through Live Link Hub.
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSettings, DefaultLiveLinkPreset));
}
