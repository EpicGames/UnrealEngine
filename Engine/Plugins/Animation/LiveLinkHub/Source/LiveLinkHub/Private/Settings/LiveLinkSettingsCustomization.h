// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

/**
 * Customization for the standard Live Link settings, specifically for Live Link Hub.
 * If we ever need to customize these core settings under the LiveLink plugin, then this class
 * may need to be moved there, and either extended here or made configurable to allow certain properties hidden by LLH.
 */
class FLiveLinkSettingsCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};