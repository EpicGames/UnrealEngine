// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHub.h"
#include "LiveLinkHubModule.h"
#include "LiveLinkTypes.h"
#include "Modules/ModuleManager.h"
#include "Session/LiveLinkHubSession.h"
#include "Session/LiveLinkHubSessionManager.h"
#include "SLiveLinkHubSubjectView.h"

/** Controller responsible for handling the hub's subjects and creating the subject view. */
class FLiveLinkHubSubjectController
{
public:
	FLiveLinkHubSubjectController()
	{
		const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
		LiveLinkHubModule.GetSessionManager()->OnActiveSessionChanged().AddRaw(this, &FLiveLinkHubSubjectController::OnActiveSessionChanged);
	}

	~FLiveLinkHubSubjectController()
	{
		if (const FLiveLinkHubModule* LiveLinkHubModule = FModuleManager::Get().GetModulePtr<FLiveLinkHubModule>("LiveLinkHub"))
		{
			if (const TSharedPtr<ILiveLinkHubSessionManager> SessionManager = LiveLinkHubModule->GetSessionManager())
			{
				SessionManager->OnActiveSessionChanged().RemoveAll(this);
			}
		}
	}

	/** Create the widget for displaying a subject's settings. */
	TSharedRef<SWidget> MakeSubjectView()
	{
		return SAssignNew(SubjectsView, SLiveLinkHubSubjectView);
	}

	/** Set the displayed subject in the subject view. */
	void SetSubject(const FLiveLinkSubjectKey& Subject) const
	{
		if (SubjectsView)
		{
			SubjectsView->SetSubject(Subject);
		}
	}

	/** Handle updating the subject details when the session has been swapped out for a different one. */
	void OnActiveSessionChanged(const TSharedRef<ILiveLinkHubSession>& ActiveSession) const
	{
		if (SubjectsView)
		{
			SubjectsView->RefreshSubjectDetails(ActiveSession);
		}
	}

private:
	/** View widget for the selected subject. */
	TSharedPtr<SLiveLinkHubSubjectView> SubjectsView;
};
