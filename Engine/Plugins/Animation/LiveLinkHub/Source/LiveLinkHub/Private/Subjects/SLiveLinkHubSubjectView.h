// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"
#include "LiveLinkClient.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkTypes.h"
#include "Modules/ModuleManager.h"
#include "SLiveLinkDataView.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Provides the UI that displays information about a livelink hub subject.
 */
class SLiveLinkHubSubjectView : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLiveLinkHubSubjectView) {}
	SLATE_ARGUMENT(FLiveLinkSubjectKey, SubjectKey)
	SLATE_END_ARGS()

	//~ Begin SWidget interface
	void Construct(const FArguments& InArgs)
	{
		ILiveLinkClient& LiveLinkClient = static_cast<ILiveLinkClient&>(IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		LiveLinkClient.OnLiveLinkSubjectRemoved().AddSP(this, &SLiveLinkHubSubjectView::OnSubjectRemoved);

		constexpr bool bReadOnly = false;
		SubjectView = UE::LiveLink::CreateSubjectsDetailsView(static_cast<FLiveLinkClient*>(&LiveLinkClient), bReadOnly);

		ChildSlot
		[
			SubjectView.ToSharedRef()
		];
	}

	virtual ~SLiveLinkHubSubjectView() override
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient& LiveLinkClient = static_cast<ILiveLinkClient&>(IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName));
			LiveLinkClient.OnLiveLinkSubjectRemoved().RemoveAll(this);
		}
	}

	/** Clear the subject details. */
	void RefreshSubjectDetails(const TSharedPtr<class ILiveLinkHubSession>& ActiveSession)
	{
		SetSubject(FLiveLinkSubjectKey{});
	}

	/** Set the subject to be displayed in the details view. */
	void SetSubject(const FLiveLinkSubjectKey& InSubjectKey)
	{
		SubjectKey = InSubjectKey;
		SubjectView->SetSubjectKey(SubjectKey);
	}

	void OnSubjectRemoved(FLiveLinkSubjectKey InSubjectKey)
	{
		if (InSubjectKey == SubjectKey)
		{
			SetSubject(FLiveLinkSubjectKey{});
		}
	}

private:
	/** Details for the selected subject. */
	TSharedPtr<IDetailsView> SettingsObjectDetailsView;
	/** Subject being shown. */
	FLiveLinkSubjectKey SubjectKey;

	TSharedPtr<SLiveLinkDataView> SubjectView;
};
