// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "LiveLinkHubClient.h"
#include "LiveLinkHub.h"
#include "Misc/FileHelper.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "ApplicationModeSwitcher"

class SApplicationModeSwitcher : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SApplicationModeSwitcher)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(2.0f, 3.0f, 2.0f, 3.0f))
			.OnGetMenuContent(this, &SApplicationModeSwitcher::GetModeSwitcherContent)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(ActiveModeImage, SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.AutoWidth()
				[
					SAssignNew(ActiveModeDisplayName, STextBlock)
				]
			]
		];

		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			LiveLinkHub->OnApplicationModeChanged().AddSP(this, &SApplicationModeSwitcher::OnAppModeChanged);

			if (TOptional<FLiveLinkHubAppModeInfo> AppModeInfo = LiveLinkHub->GetModeInfo(LiveLinkHub->GetCurrentMode()))
			{
				ActiveModeImage->SetImage(AppModeInfo->Icon.GetIcon());
				ActiveModeDisplayName->SetText(AppModeInfo->DisplayName);
			}
		}
	}

	virtual ~SApplicationModeSwitcher()
	{
		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			LiveLinkHub->OnApplicationModeChanged().RemoveAll(this);
		}
	}

private:
	/** Create the content of the layout button when it's oppened. */
	TSharedRef<SWidget> GetModeSwitcherContent()
	{
		FMenuBuilder MenuBuilder(true, NULL);

		TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get();
		const TArray<FName> Modes = LiveLinkHub->GetApplicationModes();
		const FName CurrentModeName = LiveLinkHub->GetCurrentMode();

		MenuBuilder.BeginSection("Layouts", LOCTEXT("LayoutsLabel", "Layouts"));
		
		for (FName Mode : Modes)
		{
			if (Mode != CurrentModeName)
			{
				if (TOptional<FLiveLinkHubAppModeInfo> AppModeInfo = LiveLinkHub->GetModeInfo(Mode))
				{
					// Tooltip text, Icon, localized name
					MenuBuilder.AddMenuEntry(AppModeInfo->DisplayName, AppModeInfo->DisplayName, AppModeInfo->Icon, FExecuteAction::CreateRaw(this, &SApplicationModeSwitcher::SetActiveMode, Mode));
				}
			}
		}

		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("UserLayouts", LOCTEXT("UserLayoutsLabel", "User Layouts"));

		TArray<FString> UserLayouts = FLiveLinkHub::Get()->GetUserLayouts();
		for (const FString& Layout : UserLayouts)
		{
			if (Layout != CurrentModeName)
			{
				if (TOptional<FLiveLinkHubAppModeInfo> AppModeInfo = LiveLinkHub->GetModeInfo(*Layout))
				{
					// Tooltip text, Icon, localized name
					MenuBuilder.AddMenuEntry(AppModeInfo->DisplayName, AppModeInfo->DisplayName, AppModeInfo->Icon, FExecuteAction::CreateRaw(this, &SApplicationModeSwitcher::SetActiveMode, FName(Layout)));
				}
			}
		}

		MenuBuilder.EndSection();

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry(
			INVTEXT("Save Layout As"),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]() { FLiveLinkHub::Get()->SaveLayoutAs(); })),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			INVTEXT("Load Layout"),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]() { FLiveLinkHub::Get()->LoadLayout(); })),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		if (TOptional<FLiveLinkHubAppModeInfo> AppModeInfo = LiveLinkHub->GetModeInfo(CurrentModeName))
		{
			if (!AppModeInfo->bUserLayout)
			{
				MenuBuilder.AddMenuEntry(
					INVTEXT("Reset Current Layout"),
					TAttribute<FText>(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([]() { FLiveLinkHub::Get()->ResetLayout(); })),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}

		if (UserLayouts.Num())
		{
			MenuBuilder.AddSubMenu(
				INVTEXT("Delete Layout"),
				TAttribute<FText>(),
				FNewMenuDelegate::CreateSP(this, &SApplicationModeSwitcher::CreateDeleteLayoutMenu),
				false,
				FSlateIcon()
			);
		}

		return MenuBuilder.MakeWidget();
	}

	/** Set the current application mode. */
	void SetActiveMode(FName ModeName)
	{
		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			LiveLinkHub->SetCurrentMode(ModeName);
		}
	}

	/** Handle app mode changing. */
	void OnAppModeChanged(FName NewModeName) const
	{
		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			if (TOptional<FLiveLinkHubAppModeInfo> AppModeInfo = LiveLinkHub->GetModeInfo(NewModeName))
			{
				ActiveModeImage->SetImage(AppModeInfo->Icon.GetIcon());
				ActiveModeDisplayName->SetText(AppModeInfo->DisplayName);
			}
		}
	}

	/** Populate the Delete Layout menu with the list of user layouts. */
	void CreateDeleteLayoutMenu(FMenuBuilder& MenuBuilder)
	{
		TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get();
		const FName CurrentModeName = LiveLinkHub->GetCurrentMode();

		TArray<FString> UserLayouts = FLiveLinkHub::Get()->GetUserLayouts();
		for (const FString& Layout : UserLayouts)
		{
			if (CurrentModeName != Layout)
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(Layout),
					TAttribute<FText>(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Layout]() { FLiveLinkHub::Get()->DeleteUserLayout(Layout); })),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}
	}

private:
	/** Holds the icon of the current mode. */
	TSharedPtr<SImage> ActiveModeImage;
	/** Holds the display name of the current mode. */
	TSharedPtr<STextBlock> ActiveModeDisplayName;
};

#undef LOCTEXT_NAMESPACE