// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LiveLinkHubLog.h"
#include "Misc/Paths.h"
#include "Settings/LiveLinkHubSettings.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubMemoryStats"

/** Widget that displays the current memory usage of the program and shows a warning if the memory usage falls above a threshold defined in the LiveLinkHub settings. */
class SLiveLinkHubMemoryStats : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkHubMemoryStats)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0, 0.0, 5.0, 0.0))
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([]() { return GetDefault<ULiveLinkHubSettings>()->bShowFrameRate ? EVisibility::Visible : EVisibility::Collapsed; })
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				[
					SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("MainFrame.DebugTools.LabelFont")))
						.ColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f))
						.Text(LOCTEXT("FrameRateLabel", "FPS: "))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				[
					SNew(STextBlock)
						.Text(TAttribute<FText>::CreateLambda([this]() { return FText::Format(INVTEXT("{0} / {1}"), CachedFrameRateText, CachedFrameTimeText); }))
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
								.Visibility_Lambda([]() { return GetDefault<ULiveLinkHubSettings>()->bShowMemoryUsage  ? EVisibility::Visible : EVisibility::Collapsed; })
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("MainFrame.DebugTools.LabelFont")))
					.ColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f))
					.Text(LOCTEXT("MemoryLabel", "Mem: "))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				[
					SNew(STextBlock)
						.Text(TAttribute<FText>::CreateLambda([this]() { return CachedMemoryText; }))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
						.ContentPadding(FMargin(0, 2.0))
						.ToolTipText(LOCTEXT("MemoryWarningTooltip", "Live Link Hub is using an unusual amount of RAM. Click here to collect a memory trace."))
						.Visibility(this, &SLiveLinkHubMemoryStats::OnGetWarningVisibility)
						.OnClicked(this, &SLiveLinkHubMemoryStats::OnClickMemoryWarning)
						.IsEnabled_Lambda([this]() { return !bCreatingTrace; })
						[
							SNew(SImage)
								.ColorAndOpacity(FLinearColor::Red)
								.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
						]
				]
			]
		];
	}

	virtual void Tick(const FGeometry & AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		UpdateMemoryStats();
		UpdateFrameTime();
		UpdateFrameRate();

		if (CachedTotalPhysicalUsedMB < GetDefault<ULiveLinkHubSettings>()->ShowMemoryWarningThresholdMB)
		{
			bWarnedUser = false;
		}
		else if (!bWarnedUser && !CurrentNotification)
		{
			WarnUser();
		}
	}

private:

	/** Update the cached frame rate text */
	void UpdateFrameRate() 
	{
		// Clamp to avoid huge averages at startup or after hitches
		const float AverageFPS = 1.0f / FSlateApplication::Get().GetAverageDeltaTime();
		const float ClampedFPS = (AverageFPS < 0.0f || AverageFPS > 4000.0f ) ? 0.0f : AverageFPS;

		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(1)
			.SetMaximumFractionalDigits(1);

		CachedFrameRateText = FText::AsNumber(ClampedFPS, &FormatOptions);
	}

	/** Update the cached frame time text. */
	void UpdateFrameTime()
	{
		// Clamp to avoid huge averages at startup or after hitches
		const float AverageMS = FSlateApplication::Get().GetAverageDeltaTime() * 1000.0f;
		const float ClampedMS = ( AverageMS < 0.0f || AverageMS > 4000.0f ) ? 0.0f : AverageMS;

		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(1)
			.SetMaximumFractionalDigits(1);
		static const FText FrameTimeFmt = FText::FromString(TEXT("{0} ms"));

		CachedFrameTimeText = FText::Format(FrameTimeFmt, FText::AsNumber( ClampedMS, &FormatOptions ));
	}

	/** Update our cached memory stats. */
	void UpdateMemoryStats()
	{
		// Only refresh process memory allocated after every so often, to reduce fixed frame time overhead
		static SIZE_T StaticLastTotalAllocated = 0;
		static int32 QueriesUntilUpdate = 1;
		if (--QueriesUntilUpdate <= 0)
		{
			// Query OS for process memory used
			FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			StaticLastTotalAllocated = MemoryStats.UsedPhysical;

			// Wait 120 queries until we refresh memory again
			QueriesUntilUpdate = FramesBetweenPlatformQueries;
		}

		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(2)
			.SetMaximumFractionalDigits(2);
		static const FText MemorySizeFmt = FText::FromString(TEXT("{0} mb"));

		CachedTotalPhysicalUsedMB = (float)StaticLastTotalAllocated / (1024.0f * 1024.0f);
		CachedMemoryText = FText::Format(MemorySizeFmt, FText::AsNumber(CachedTotalPhysicalUsedMB, &FormatOptions));

		if (CurrentNotification && CurrentNotification->GetCompletionState() == SNotificationItem::CS_None)
		{
			CurrentNotification->SetText(CreateWarningText());
		}
	}

	/** Open a toast notification to warn the user that memory consumption is unusually high. */
	void WarnUser()
	{
		FNotificationInfo Info(CreateWarningText());
		Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
		Info.bFireAndForget = false;
		Info.WidthOverride = 500;

		FNotificationButtonInfo CreateReportButton{ LOCTEXT("CreateTraceButton", "Create trace snapshot"), LOCTEXT("CreateTraceButtonToolTip", "Generate a trace snapshot."), FSimpleDelegate::CreateSP(this, &SLiveLinkHubMemoryStats::GenerateTraceSnapshot), SNotificationItem::ECompletionState::CS_None };
		FNotificationButtonInfo CloseNotificationButton{ LOCTEXT("CloseNotificationButton", "Close"), FText::GetEmpty(), FSimpleDelegate::CreateSP(this, &SLiveLinkHubMemoryStats::CloseNotification), SNotificationItem::ECompletionState::CS_None };
		Info.ButtonDetails.Add(CreateReportButton);
		Info.ButtonDetails.Add(CloseNotificationButton);

		CurrentNotification = FSlateNotificationManager::Get().AddNotification(Info);
		CurrentNotification->SetCompletionState(SNotificationItem::CS_None);

		bWarnedUser = true;
	}

	/** Handles clicking on the warning button to create a trace snapshot. */
	FReply OnClickMemoryWarning()
	{
		GenerateTraceSnapshot();

		return FReply::Handled();
	}

	/** Handles creating a snapshot tracefile. */
	void GenerateTraceSnapshot()
	{
		if (bCreatingTrace)
		{
			return;
		}

		bCreatingTrace = true;

		if (CurrentNotification)
		{
			CurrentNotification->SetText(LOCTEXT("CreatedTrace", "Created trace snapshot..."));
			CurrentNotification->SetCompletionState(SNotificationItem::CS_Success);
			CurrentNotification->ExpireAndFadeout();
		}

		const FString TraceFileName = FDateTime::Now().ToString(TEXT("LiveLinkHubMemoryLeak - %Y%m%d_%H%M%S.utrace"));
		LastTracePath = FPaths::Combine(FPaths::ProfilingDir(), TraceFileName);

		if (!UE::Trace::WriteSnapshotTo(*LastTracePath))
		{
			UE_LOGF(LogLiveLinkHub, Warning, "Failed writing trace snapshot file.");
		}
		else
		{
			UE_LOGF(LogLiveLinkHub, Display, "Wrote trace snapshot file to %ls", *LastTracePath);
		}


		bCreatingTrace = false;
	}

	/** Closes the current memory usage notification. */
	void CloseNotification()
	{
		CurrentNotification->Fadeout();
		CurrentNotification.Reset();
	}

	/** Determines whether the warning icon should be visible. */
	EVisibility OnGetWarningVisibility() const
	{
		if (CachedTotalPhysicalUsedMB > GetDefault<ULiveLinkHubSettings>()->ShowMemoryWarningThresholdMB)
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	}

	/** Generate the warning message for current memory usage. */
	FText CreateWarningText()
	{
		const FText WarningMessage = FText::Format(LOCTEXT("PhysicalMemoryWarning", "Physical memory usage is unusually high! ({0})."), CachedMemoryText);
		return WarningMessage;
	}

private:
	/** Cached memory used in this process. */
	float CachedTotalPhysicalUsedMB = 0.f;
	/** Cached text representation of the current app frame rate. */
	FText CachedFrameRateText;
	/** Cached text representation of the current app frame time. */
	FText CachedFrameTimeText;
	/** Cached text representation of our total physical memory used. */
	FText CachedMemoryText;
	/** Stores the path to the last trace snapshot that was generated. */
	FString LastTracePath;
	/** Whether the hub is currently generating a report. */
	bool bCreatingTrace = false;
	/** Whether the user was warned about high memory usage (will reset once it goes below threshold again. */
	bool bWarnedUser = false;
	/** Pointer to the current notification shown on screen. */
	TSharedPtr<SNotificationItem> CurrentNotification;
	/** Number of frames to wait between calls to FPlatformMemory::GetStats */
	const int32 FramesBetweenPlatformQueries = 120;
};

#undef LOCTEXT_NAMESPACE