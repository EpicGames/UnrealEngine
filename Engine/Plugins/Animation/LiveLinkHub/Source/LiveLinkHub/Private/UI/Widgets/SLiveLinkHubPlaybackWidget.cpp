// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkHubPlaybackWidget.h"

#include "Recording/LiveLinkRecordingRangeHelpers.h"
#include "SLiveLinkHubTimeSlider.h"

#include "FrameNumberNumericInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SSimpleTimeSlider.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SLiveLinkHubPlaybackWidget"

void SLiveLinkHubPlaybackWidget::Construct(const FArguments& InArgs)
{
	OnPlayForwardDelegate = InArgs._OnPlayForward;
	OnPlayReverseDelegate = InArgs._OnPlayReverse;

	OnFirstFrameDelegate = InArgs._OnFirstFrame;
	OnLastFrameDelegate = InArgs._OnLastFrame;

	OnPreviousFrameDelegate = InArgs._OnPreviousFrame;
	OnNextFrameDelegate = InArgs._OnNextFrame;

	OnSetPlayRateDelegate = InArgs._OnSetPlayRate;

	OnExitPlaybackDelegate = InArgs._OnExitPlayback;
	OnSelectRecordingDelegate = InArgs._OnSelectRecording;

	OnGetPausedDelegate = InArgs._IsPaused;
	OnGetIsInReverseDelegate = InArgs._IsInReverse;

	OnGetCurrentTimeDelegate = InArgs._GetCurrentTime;
	OnGetTotalLengthDelegate = InArgs._GetTotalLength;

	OnSetCurrentTimeDelegate = InArgs._SetCurrentTime;

	OnGetViewRangeDelegate = InArgs._GetViewRange;
	OnSetViewRangeDelegate = InArgs._SetViewRange;

	OnGetSelectionStartTimeDelegate = InArgs._GetSelectionStartTime;
	OnGetSelectionEndTimeDelegate = InArgs._GetSelectionEndTime;
	OnSetSelectionStartTimeDelegate = InArgs._SetSelectionStartTime;
	OnSetSelectionEndTimeDelegate = InArgs._SetSelectionEndTime;
	
	OnSetLoopingDelegate = InArgs._OnSetLooping;
	OnGetLoopingDelegate = InArgs._IsLooping;

	IsPlaybackEnabledDelegate = InArgs._IsPlaybackEnabled;

	IsUpgradingDelegate = InArgs._IsUpgrading;
	OnGetUpgradePercentDelegate = InArgs._OnGetUpgradePercent;

	OnGetFrameRate = InArgs._GetFrameRate;

	OnScrubbingStartDelegate = InArgs._OnScrubbingStart;
	OnScrubbingEndDelegate = InArgs._OnScrubbingEnd;

	OnGetFrameBufferRanges = InArgs._GetBufferRanges;

	const TAttribute<EFrameNumberDisplayFormats> GetDisplayFormatAttr = MakeAttributeSP(this, &SLiveLinkHubPlaybackWidget::GetDisplayFormat);
	const TAttribute<FFrameRate> GetDisplayRateAttr = MakeAttributeSP(this, &SLiveLinkHubPlaybackWidget::GetFrameRate);
	TAttribute<bool> IsPlaybackEnabled = MakeAttributeSP(this, &SLiveLinkHubPlaybackWidget::IsPlaybackEnabled);
	
	// Create our numeric type interface so we can pass it to the time slider below.
	NumberInterface = MakeShareable(new FFrameNumberInterface(GetDisplayFormatAttr, 0, GetDisplayRateAttr, GetDisplayRateAttr));

	const ISlateStyle* LiveLinkStyle = FSlateStyleRegistry::FindSlateStyle("LiveLinkStyle");

	NumberInterface->DisplayFormatChanged();
	ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16.f, 8.f)
			[
				// Box is just so we can wrap the playback control to disable it. We don't want to disable
				// the entire vertical box because we want the eject option to remain available.
				SNew(SBox)
				.IsEnabled(IsPlaybackEnabled)
				[
					SAssignNew(TimeSlider, SLiveLinkHubTimeSlider)
					.IsEnabled(IsPlaybackEnabled)
					.BaseArgs(SSimpleTimeSlider::FArguments()
					.ClampRangeHighlightSize(0.35f)
					.ClampRangeHighlightColor(FLinearColor::Gray.CopyWithNewOpacity(0.5f))
					.ScrubPosition_Lambda([this]()
					{
						// TimeSlider needs actual time in seconds, not the frame time.
						const FQualifiedFrameTime Now = GetCurrentTime();
						return Now.AsSeconds();
					})
					.ViewRange(this, &SLiveLinkHubPlaybackWidget::GetViewRange)
					.OnViewRangeChanged(this, &SLiveLinkHubPlaybackWidget::SetViewRange)
					.ClampRange(this, &SLiveLinkHubPlaybackWidget::GetClampRange)
					.OnBeginScrubberMovement_Lambda([this]() { OnScrubbingStartDelegate.ExecuteIfBound(); })
					.OnEndScrubberMovement_Lambda([this]() { OnScrubbingEndDelegate.ExecuteIfBound(); })
					.OnScrubPositionChanged_Lambda(
						[this](double NewScrubTime, bool bIsScrubbing)
								{
									if (bIsScrubbing)
									{
										//  Convert time in seconds to frame time as double.
										SetCurrentTime(NewScrubTime);
									}
								})
					)
					.BufferRange(this, &SLiveLinkHubPlaybackWidget::GetBufferRanges)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.f, 8.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnFirstFramePressed)
					.ToolTipText(LOCTEXT("ToFront", "To Front"))
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush("Animation.Backward_End"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnPreviousFramePressed)
					.ToolTipText(LOCTEXT("ToPrevious", "To Previous"))
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush("Animation.Backward_Step"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnPlayReversePressed)
					.ToolTipText_Lambda([this]()
					{
						return IsPaused() || !IsPlayingInReverse() ? LOCTEXT("ReverseButton", "Reverse") : LOCTEXT("PauseButton", "Pause");
					})
					.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(this, &SLiveLinkHubPlaybackWidget::GetPlayReverseIcon)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnPlayForwardPressed)
					.ToolTipText_Lambda([this]()
					{
						return IsPaused() || IsPlayingInReverse() ? LOCTEXT("PlayButton", "Play") : LOCTEXT("PauseButton", "Pause");
					})
					.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(this, &SLiveLinkHubPlaybackWidget::GetPlayForwardIcon)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnNextFramePressed)
					.ToolTipText(LOCTEXT("ToNext", "To Next"))
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush("Animation.Forward_Step"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnLastFramePressed)
					.ToolTipText(LOCTEXT("ToEnd", "To End"))
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush("Animation.Forward_End"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(IsPlaybackEnabled)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnLoopPressed)
					.ToolTipText_Raw(this, &SLiveLinkHubPlaybackWidget::GetLoopTooltip)
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(this, &SLiveLinkHubPlaybackWidget::GetLoopIcon)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(2.0))
					.IsEnabled(IsPlaybackEnabled)
					.HasDownArrow(true)
					.OnGetMenuContent(this, &SLiveLinkHubPlaybackWidget::MakePlayRateMenuContent)
					.ForegroundColor(FSlateColor::UseStyle())
					.ToolTipText(LOCTEXT("PlaybackRate_Tooltip", "Change Play Rate."))
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text_Lambda([this]() { return FText::Format(INVTEXT("{0}x"), CachedPlayRate); })
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0, 0.0, 0.0, 0.0))
				.AutoWidth()
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([IsPlaybackEnabled]() { return IsPlaybackEnabled.Get() ? 0 : 1; })
					+ SWidgetSwitcher::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
						.ContentPadding(FMargin(2.0f))
						.IsEnabled(IsPlaybackEnabled)
						.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnExitPlaybackPressed)
						.ToolTipText(LOCTEXT("EjectButtonToolTip", "Exit Playback"))
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("LevelViewport.EjectActorPilot.Small"))
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]
					]
					+ SWidgetSwitcher::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
						.ContentPadding(FMargin(2.0f))
						.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnSelectRecordingPressed)
						.ToolTipText(LOCTEXT("SelectRecordingToolTip", "Select a recording."))
						[
							SNew(SImage)
							.Image(LiveLinkStyle->GetBrush(TEXT("LiveLinkHub.Playback.Icon")))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]
					]
				]

				+SHorizontalBox::Slot()
				.Padding(12.f, 0.f, 0.f, 0.f)
				[
					SNew(SSpinBox<double>)
					.IsEnabled(IsPlaybackEnabled)
					.Value(this, &SLiveLinkHubPlaybackWidget::GetSelectionStartTime)
					.ToolTipText(LOCTEXT("SelectionStartTime", "Selection start time"))
					.OnValueChanged(this, &SLiveLinkHubPlaybackWidget::SetSelectionStartTime)
					.MinValue(TOptional<double>())
					.MaxValue(TOptional<double>())
					.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.TypeInterface(NumberInterface)
					.ClearKeyboardFocusOnCommit(true)
					.Delta(this, &SLiveLinkHubPlaybackWidget::GetSpinboxDelta)
					.LinearDeltaSensitivity(25)
				]
				+SHorizontalBox::Slot()
				.Padding(4.f, 0.f)
				[
					SNew(SSpinBox<double>)
					.IsEnabled(IsPlaybackEnabled)
					.Value(this, &SLiveLinkHubPlaybackWidget::GetCurrentTimeDouble)
					.ToolTipText(LOCTEXT("CurrentTime", "Current playback time"))
					.OnValueChanged(this, &SLiveLinkHubPlaybackWidget::SetCurrentTime)
					.MinValue(TOptional<double>())
					.MaxValue(TOptional<double>())
					.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.TypeInterface(NumberInterface)
					.ClearKeyboardFocusOnCommit(true)
					.Delta(this, &SLiveLinkHubPlaybackWidget::GetSpinboxDelta)
					.LinearDeltaSensitivity(25)
				]
				+SHorizontalBox::Slot()
				.Padding(4.f, 0.f)
				[
					SNew(SSpinBox<double>)
					.IsEnabled(IsPlaybackEnabled)
					.Value(this, &SLiveLinkHubPlaybackWidget::GetSelectionEndTime)
					.ToolTipText(LOCTEXT("SelectionEndTime", "Selection end time"))
					.OnValueChanged(this, &SLiveLinkHubPlaybackWidget::SetSelectionEndTime)
					.MinValue(TOptional<double>())
					.MaxValue(TOptional<double>())
					.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.TypeInterface(NumberInterface)
					.ClearKeyboardFocusOnCommit(true)
					.Delta(this, &SLiveLinkHubPlaybackWidget::GetSpinboxDelta)
					.LinearDeltaSensitivity(25)
				]
				+SHorizontalBox::Slot()
				.Padding(4.f, 0.f)
				[
					SNew(SSpinBox<double>)
					.IsEnabled(IsPlaybackEnabled)
					.Value(this, &SLiveLinkHubPlaybackWidget::GetTotalLength)
					.IsEnabled(false)
					.ToolTipText(LOCTEXT("RecordingLength", "Recording length"))
					.MinValue(TOptional<double>())
					.MaxValue(TOptional<double>())
					.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.TypeInterface(NumberInterface)
				]
				+SHorizontalBox::Slot()
				.Padding(4.f, 0.f)
				.AutoWidth()
				[
					SNew(SComboButton)
					.IsEnabled(IsPlaybackEnabled)
					.HasDownArrow(true)
					.OnGetMenuContent(this, &SLiveLinkHubPlaybackWidget::MakePlaybackSettingsDropdown)
					.ForegroundColor(FSlateColor::UseStyle())
					.ToolTipText(LOCTEXT("PlaybackSettings_Tooltip", "Change playback settings."))
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				]
			]
		]
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			MakeUpgradingWidget()
		]
	];
}

void SLiveLinkHubPlaybackWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!IsPlaybackEnabled())
	{
		return;
	}

	if (KeyDown[(uint8)EScrubDirection::Left])
	{
		KeyDownCounter[(uint8)EScrubDirection::Left]++;
	}

	if (KeyDown[(uint8)EScrubDirection::Right])
	{
		KeyDownCounter[(uint8)EScrubDirection::Right]++;
	}

	if (KeyDownCounter[(uint8)EScrubDirection::Left] > MinFramesToBeginScrubbing)
	{
		// This controls the rate of play when scrubbing.
		if (KeyDownScrubCounter++ >= FramesBetweenKeyDownScrub)
		{
			OnPreviousFramePressed();
			KeyDownScrubCounter = 0;
		}
	}
	else if (KeyDownCounter[(uint8)EScrubDirection::Right] > MinFramesToBeginScrubbing)
	{
		// This controls the rate of play when scrubbing.
		if (KeyDownScrubCounter++ >= FramesBetweenKeyDownScrub)
		{
			OnNextFramePressed();
			KeyDownScrubCounter = 0;
		}
	}
}

FReply SLiveLinkHubPlaybackWidget::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	// Forward focus to the time slider widget to allow 
	if (InFocusEvent.GetCause() == EFocusCause::SetDirectly)
	{
		FSlateApplication::Get().SetUserFocus(0, TimeSlider);
	}

	return FReply::Handled();
}

void SLiveLinkHubPlaybackWidget::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	KeyDown[(uint8)EScrubDirection::Left] = false;
	KeyDown[(uint8)EScrubDirection::Right] = false;

	KeyDownCounter[(uint8)EScrubDirection::Left] = 0;
	KeyDownCounter[(uint8)EScrubDirection::Right] = 0;
	KeyDownScrubCounter = 0;
}

FReply SLiveLinkHubPlaybackWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Left || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		KeyDown[(uint8)EScrubDirection::Left] = true;
	}
	else if (InKeyEvent.GetKey() == EKeys::Right)
	{
		KeyDown[(uint8)EScrubDirection::Right] = true;
	}

	return FReply::Unhandled();
}

FReply SLiveLinkHubPlaybackWidget::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Left || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		KeyDown[(uint8)EScrubDirection::Left] = false;
		KeyDownCounter[(uint8)EScrubDirection::Left] = 0;
		KeyDownScrubCounter = 0;
		return OnPreviousFramePressed().SetUserFocus(AsShared()); // Set focus to prevent navigating on buttons
	}
	else if (InKeyEvent.GetKey() == EKeys::Right)
	{
		KeyDown[(uint8)EScrubDirection::Right] = false;
		KeyDownCounter[(uint8)EScrubDirection::Right] = 0;
		KeyDownScrubCounter = 0;
		return OnNextFramePressed().SetUserFocus(AsShared()); // Set focus to prevent navigating on buttons
	}
	else if (InKeyEvent.GetKey() == EKeys::SpaceBar || InKeyEvent.GetKey() == EKeys::Enter)
	{
		OnPlayForwardPressed();
	}

	return FReply::Unhandled();
}

FReply SLiveLinkHubPlaybackWidget::OnPlayForwardPressed()
{
	OnPlayForwardDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnPlayReversePressed()
{
	OnPlayReverseDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnFirstFramePressed()
{
	OnFirstFrameDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnLastFramePressed()
{
	OnLastFrameDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnPreviousFramePressed()
{
	OnPreviousFrameDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnNextFramePressed()
{
	OnNextFrameDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnLoopPressed()
{
	check(OnSetLoopingDelegate.IsBound() && OnGetLoopingDelegate.IsBound());
	OnSetLoopingDelegate.Execute(!OnGetLoopingDelegate.Execute());
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnExitPlaybackPressed()
{
	OnExitPlaybackDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

FReply SLiveLinkHubPlaybackWidget::OnSelectRecordingPressed()
{
	OnSelectRecordingDelegate.ExecuteIfBound();
	return HandleAndFocusTimeSlider();
}

void SLiveLinkHubPlaybackWidget::SetCurrentTime(double InTime)
{
	check(OnSetCurrentTimeDelegate.IsBound());

	const FQualifiedFrameTime FrameTime = SecondsToFrameTime(InTime);

	OnSetCurrentTimeDelegate.Execute(FrameTime);
}

double SLiveLinkHubPlaybackWidget::GetCurrentTimeDouble() const
{
	check(OnGetCurrentTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = OnGetCurrentTimeDelegate.Execute();
		
	return FrameTime.AsSeconds();
}

FQualifiedFrameTime SLiveLinkHubPlaybackWidget::GetCurrentTime() const
{
	check(OnGetCurrentTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = OnGetCurrentTimeDelegate.Execute();
		
	return FrameTime;
}

double SLiveLinkHubPlaybackWidget::GetTotalLength() const
{
	check(OnGetTotalLengthDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = OnGetTotalLengthDelegate.Execute();
	return FrameTime.AsSeconds();
}

double SLiveLinkHubPlaybackWidget::GetSelectionStartTime() const
{
	check(OnGetSelectionStartTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = OnGetSelectionStartTimeDelegate.Execute();
	return FrameTime.AsSeconds();
}

void SLiveLinkHubPlaybackWidget::SetSelectionStartTime(double InTime)
{
	check(OnSetSelectionStartTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = SecondsToFrameTime(InTime);
	OnSetSelectionStartTimeDelegate.Execute(FrameTime);
}

double SLiveLinkHubPlaybackWidget::GetSelectionEndTime() const
{
	check(OnGetSelectionEndTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = OnGetSelectionEndTimeDelegate.Execute();

	return FrameTime.AsSeconds();
}

void SLiveLinkHubPlaybackWidget::SetSelectionEndTime(double InTime)
{
	check(OnSetSelectionEndTimeDelegate.IsBound());
	const FQualifiedFrameTime FrameTime = SecondsToFrameTime(InTime);
	OnSetSelectionEndTimeDelegate.Execute(FrameTime);
}

TRange<double> SLiveLinkHubPlaybackWidget::GetViewRange() const
{
	check(OnGetViewRangeDelegate.IsBound());
	return OnGetViewRangeDelegate.Execute();
}

void SLiveLinkHubPlaybackWidget::SetViewRange(TRange<double> InRange)
{
	check(OnSetViewRangeDelegate.IsBound());
	OnSetViewRangeDelegate.Execute(InRange);
}

TRange<double> SLiveLinkHubPlaybackWidget::GetClampRange() const
{
	const double Start = GetSelectionStartTime();
	const double End = GetSelectionEndTime();
	return UE::LiveLinkHub::RangeHelpers::Private::MakeInclusiveRange(Start, End);
}

UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<double> SLiveLinkHubPlaybackWidget::GetBufferRanges() const
{
	using namespace UE::LiveLinkHub::RangeHelpers::Private;
	check(OnGetFrameBufferRanges.IsBound());
	
	TRangeArray<double> OutputRanges;
	
	const TRangeArray<int32> BufferedFrameRanges = OnGetFrameBufferRanges.Execute();
	for (const TRange<int32>& BufferedFrames : BufferedFrameRanges)
	{
		FQualifiedFrameTime StartTime(BufferedFrames.GetLowerBound().GetValue(), GetFrameRate());
		FQualifiedFrameTime EndTime(BufferedFrames.GetUpperBound().GetValue(), GetFrameRate());
		OutputRanges.Add(MakeInclusiveRange(StartTime.AsSeconds(), EndTime.AsSeconds()));
	}

	return OutputRanges;
}

bool SLiveLinkHubPlaybackWidget::IsPaused() const
{
	return OnGetPausedDelegate.IsBound() ? OnGetPausedDelegate.Execute() : false;
}

bool SLiveLinkHubPlaybackWidget::IsPlayingInReverse() const
{
	return OnGetIsInReverseDelegate.IsBound() ? OnGetIsInReverseDelegate.Execute() : false;
}

double SLiveLinkHubPlaybackWidget::GetSpinboxDelta() const
{
	return GetFrameRate().AsDecimal() * GetFrameRate().AsInterval();
}

EFrameNumberDisplayFormats SLiveLinkHubPlaybackWidget::GetDisplayFormat() const
{
	return DisplayFormat;
}

void SLiveLinkHubPlaybackWidget::SetDisplayFormat(EFrameNumberDisplayFormats InDisplayFormat)
{
	DisplayFormat = InDisplayFormat;
	NumberInterface->DisplayFormatChanged();
}

bool SLiveLinkHubPlaybackWidget::CompareDisplayFormat(EFrameNumberDisplayFormats InDisplayFormat) const
{
	return DisplayFormat == InDisplayFormat;
}

void SLiveLinkHubPlaybackWidget::SetPlayRate(float InPlayRate)
{
	CachedPlayRate = FMath::Clamp(InPlayRate, 0.01, 5);
	OnSetPlayRateDelegate.ExecuteIfBound(InPlayRate);
}

FText SLiveLinkHubPlaybackWidget::GetDisplayFormatAsText() const
{
	return DisplayFormat == EFrameNumberDisplayFormats::Frames ? LOCTEXT("DisplayFormat_TimeFrames", "Frames") :
		DisplayFormat == EFrameNumberDisplayFormats::Seconds ? LOCTEXT("DisplayFormat_TimeSeconds", "Seconds") :
		LOCTEXT("DisplayFormat_Timecode", "Timecode") ;
}

FFrameRate SLiveLinkHubPlaybackWidget::GetFrameRate() const
{
	check(OnGetFrameRate.IsBound());
	return OnGetFrameRate.Execute();
}

FQualifiedFrameTime SLiveLinkHubPlaybackWidget::SecondsToFrameTime(double InTime) const
{
	const FFrameRate FrameRate = GetFrameRate();
	return FQualifiedFrameTime(FrameRate.AsFrameTime(InTime), FrameRate);
}

const FSlateBrush* SLiveLinkHubPlaybackWidget::GetPlayForwardIcon() const
{
	return IsPaused() || IsPlayingInReverse() ? FAppStyle::Get().GetBrush("Animation.Forward") : FAppStyle::Get().GetBrush("Animation.Pause");
}

const FSlateBrush* SLiveLinkHubPlaybackWidget::GetPlayReverseIcon() const
{
	return IsPaused() || !IsPlayingInReverse() ? FAppStyle::Get().GetBrush("Animation.Backward") : FAppStyle::Get().GetBrush("Animation.Pause");
}

const FSlateBrush* SLiveLinkHubPlaybackWidget::GetLoopIcon() const
{
	check(OnGetLoopingDelegate.IsBound());
	const bool bLooping = OnGetLoopingDelegate.Execute();
	return bLooping ? FAppStyle::Get().GetBrush("Animation.Loop.Enabled")
		: FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
}

FText SLiveLinkHubPlaybackWidget::GetLoopTooltip() const
{
	check(OnGetLoopingDelegate.IsBound());
	const bool bLooping = OnGetLoopingDelegate.Execute();

	return bLooping ? LOCTEXT("Loop", "Loop") : LOCTEXT("NoLoop", "No looping");
}

TSharedRef<SWidget> SLiveLinkHubPlaybackWidget::MakePlaybackSettingsDropdown()
{
	constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	const FCanExecuteAction AlwaysExecute = FCanExecuteAction::CreateLambda([]{ return true; });

	MenuBuilder.BeginSection("ShowTimeAsSection", LOCTEXT("ShowTimeAs", "Show Time As"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Menu_TimecodeLabel", "Timecode"),
		LOCTEXT("Menu_TimecodeTooltip", "Display values in timecode format."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SLiveLinkHubPlaybackWidget::SetDisplayFormat, EFrameNumberDisplayFormats::NonDropFrameTimecode),
			AlwaysExecute,
			FIsActionChecked::CreateSP(this, &SLiveLinkHubPlaybackWidget::CompareDisplayFormat, EFrameNumberDisplayFormats::NonDropFrameTimecode)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Menu_FramesLabel", "Frames"),
		LOCTEXT("Menu_FramesTooltip", "Display values as frame numbers."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SLiveLinkHubPlaybackWidget::SetDisplayFormat, EFrameNumberDisplayFormats::Frames),
			AlwaysExecute,
			FIsActionChecked::CreateSP(this, &SLiveLinkHubPlaybackWidget::CompareDisplayFormat, EFrameNumberDisplayFormats::Frames)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Menu_SecondsLabel", "Seconds"),
		LOCTEXT("Menu_SecondsTooltip", "Display values in seconds."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SLiveLinkHubPlaybackWidget::SetDisplayFormat, EFrameNumberDisplayFormats::Seconds),
			AlwaysExecute,
			FIsActionChecked::CreateSP(this, &SLiveLinkHubPlaybackWidget::CompareDisplayFormat, EFrameNumberDisplayFormats::Seconds)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
	
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SLiveLinkHubPlaybackWidget::MakePlayRateMenuContent()
{
	constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	const FCanExecuteAction AlwaysExecute = FCanExecuteAction::CreateLambda([]{ return true; });

	MenuBuilder.BeginSection("PlayRateSection", LOCTEXT("PlayRate", "Play Rate"));

	TArray<float, TInlineAllocator<6>> Values = { 0.25f, 0.5f, 0.75f, 1.0f, 1.25f, 1.5f, 2.0f };

	for (float Value : Values)
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(INVTEXT("{0}x"), Value),
			FText::Format(LOCTEXT("PlayRateToolTip", "Set Play Rate to {0}x"), Value),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLiveLinkHubPlaybackWidget::SetPlayRate, Value),
				AlwaysExecute,
				FIsActionChecked::CreateLambda([this, Value]() -> bool { return Value == CachedPlayRate; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SLiveLinkHubPlaybackWidget::MakeUpgradingWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Visibility_Lambda([this]()
		{
			if (IsUpgradingDelegate.IsBound() && IsUpgradingDelegate.Execute())
			{
				return EVisibility::Visible;
			}

			return EVisibility::Collapsed;
		})
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(6.f, 0.f)
				.AutoWidth()
				[
					SNew(SCircularThrobber)
					.Radius(6.f)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SLiveLinkHubPlaybackWidget::GetUpgradeText)
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0.f, 8.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SLiveLinkHubPlaybackWidget::OnExitPlaybackPressed)
				.Text(LOCTEXT("CancelUpgradeText", "Cancel"))
				.ToolTipText(LOCTEXT("UpgradeCancelToolTip", "Cancel upgrade and eject the recording."))
			]
		];
}

FText SLiveLinkHubPlaybackWidget::GetUpgradeText() const
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.MaximumFractionalDigits = 0;
	
	const float UpgradePercent = GetUpgradePercent();
	FText Text = FText::Format(LOCTEXT("UpgradeText", "Upgrading recording... {0}%"),
		FText::AsNumber(UpgradePercent, &NumberFormatOptions));
	return Text;
}

float SLiveLinkHubPlaybackWidget::GetUpgradePercent() const
{
	check(OnGetUpgradePercentDelegate.IsBound());
	return OnGetUpgradePercentDelegate.Execute();
}

bool SLiveLinkHubPlaybackWidget::IsPlaybackEnabled() const
{
	return IsPlaybackEnabledDelegate.Execute();
}

FReply SLiveLinkHubPlaybackWidget::HandleAndFocusTimeSlider() const
{
	return FReply::Handled().SetUserFocus(TimeSlider.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
