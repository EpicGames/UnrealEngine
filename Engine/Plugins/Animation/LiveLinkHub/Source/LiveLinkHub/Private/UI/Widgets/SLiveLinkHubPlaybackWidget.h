// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameNumberDisplayFormat.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "Recording/LiveLinkRecordingRangeHelpers.h"
#include "Widgets/SCompoundWidget.h"

class FLiveLinkHubPlaybackController;
class ULiveLinkRecording;
struct FFrameNumberInterface;

/**
 * The playback widget for controlling live link hub animations.
 */
class SLiveLinkHubPlaybackWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(bool, FOnGetChecked);
	DECLARE_DELEGATE_OneParam(FOnSetChecked, bool);
	DECLARE_DELEGATE_RetVal(FQualifiedFrameTime, FOnGetTime);
	DECLARE_DELEGATE_OneParam(FOnSetTime, FQualifiedFrameTime);
	DECLARE_DELEGATE_RetVal(FFrameRate, FOnGetFrame);
	DECLARE_DELEGATE_OneParam(FOnSetDoubleRange, const TRange<double>&)
	DECLARE_DELEGATE_RetVal(TRange<double>, FOnGetDoubleRange)
	DECLARE_DELEGATE_RetVal(UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<int32>, FOnGetIntArrayRange)
	DECLARE_DELEGATE_RetVal(float, FOnGetFloat)
	DECLARE_DELEGATE_OneParam(FOnSetFloat, float)
	
	DECLARE_DELEGATE(FOnButtonPressed);
	
	SLATE_BEGIN_ARGS(SLiveLinkHubPlaybackWidget) { }
	/** When forward play is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnPlayForward)
	/** When reverse play is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnPlayReverse)
	/** When go to first frame is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnFirstFrame)
	/** When go to last frame is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnLastFrame)
	/** When previous frame is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnPreviousFrame)
	/** When next frame is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnNextFrame)
	/** When next frame is pressed. */
	SLATE_EVENT(FOnSetFloat, OnSetPlayRate)
	/** When the eject button is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnExitPlayback)
	/** When the select recording button is pressed. */
	SLATE_EVENT(FOnButtonPressed, OnSelectRecording)
	/** Is the recording looping? */
	SLATE_EVENT(FOnGetChecked, IsLooping)
	/** Set the loop state of the recording. */
	SLATE_EVENT(FOnSetChecked, OnSetLooping)
	/** Get if the playback widget should be enabled.*/
	SLATE_EVENT(FOnGetChecked, IsPlaybackEnabled)
	/** If the recording is upgrading. */
	SLATE_EVENT(FOnGetChecked, IsUpgrading)
	/** Get the upgrade percent. */
	SLATE_EVENT(FOnGetFloat, OnGetUpgradePercent)
	/** Is the recording paused? */
	SLATE_EVENT(FOnGetChecked, IsPaused)
	/** Is the recording playing in reverse? */
	SLATE_EVENT(FOnGetChecked, IsInReverse)
	/** Get the current time of the recording. */
	SLATE_EVENT(FOnGetTime, GetCurrentTime)
	/** Set the current time of the recording. */
	SLATE_EVENT(FOnSetTime, SetCurrentTime)
	/** Get the total length of the recording. */
	SLATE_EVENT(FOnGetTime, GetTotalLength)
	/** Get the selection start of the recording. */
	SLATE_EVENT(FOnGetTime, GetSelectionStartTime)
	/** Set the selection start of the recording. */
	SLATE_EVENT(FOnSetTime, SetSelectionStartTime)
	/** Get the selection end of the recording. */
	SLATE_EVENT(FOnGetTime, GetSelectionEndTime)
	/** Set the selection end of the recording. */
	SLATE_EVENT(FOnSetTime, SetSelectionEndTime)
	/** Retrieve the frame rate for the recording. */
	SLATE_EVENT(FOnGetFrame, GetFrameRate)

	/** Delegate fired when scrubbing starts. */
	SLATE_EVENT(FSimpleDelegate, OnScrubbingStart)
	/** Delegate fired when scrubbing ends. */
	SLATE_EVENT(FSimpleDelegate, OnScrubbingEnd)
	
	/** Get the view range (visible selection range). */
	SLATE_EVENT(FOnSetDoubleRange, SetViewRange)
	/** Set the view range (visible selection range). */
	SLATE_EVENT(FOnGetDoubleRange, GetViewRange)
	/** Retrieve the frame buffer range. */
	SLATE_EVENT(FOnGetIntArrayRange, GetBufferRanges)
		
	SLATE_END_ARGS()

	/**
	 * @param InArgs 
	 */
	void Construct(const FArguments& InArgs);

protected:
	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const { return true; }
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

private:
	/** When play forward is pressed. */
	FReply OnPlayForwardPressed();
	/** When play reverse is pressed. */
	FReply OnPlayReversePressed();
	/** When first frame is pressed. */
	FReply OnFirstFramePressed();
	/** When last frame is pressed. */
	FReply OnLastFramePressed();
	/** When previous frame is pressed. */
	FReply OnPreviousFramePressed();
	/** When next frame is pressed. */
	FReply OnNextFramePressed();
	/** When loop recording is pressed. */
	FReply OnLoopPressed();
	/** When using wishes to select a recording to play. */
	FReply OnSelectRecordingPressed();
	/** When recording is exited. */
	FReply OnExitPlaybackPressed();

	/** Sets the current playhead time. */
	void SetCurrentTime(double InTime);

	/** The current playhead time. */
	double GetCurrentTimeDouble() const;
	FQualifiedFrameTime GetCurrentTime() const;
	/** The total length of the recording. */
	double GetTotalLength() const;

	/** Retrieve the selection start time. */
	double GetSelectionStartTime() const;
	/** Set the selection start time. */
	void SetSelectionStartTime(double InTime);

	/** Retrieve the selection end time. */
	double GetSelectionEndTime() const;
	/** Set the selection end time. */
	void SetSelectionEndTime(double InTime);

	/** Retrieve the visible range of the scrubber. */
	TRange<double> GetViewRange() const;
	/** Set the visible range of the scrubber. */
	void SetViewRange(TRange<double> InRange);

	/** Retrieve the range to clamp playback to (the selection). */
	TRange<double> GetClampRange() const;

	/** Retrieve the buffered frame range. */
	UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<double> GetBufferRanges() const;
	
	/** Is playback paused? */
	bool IsPaused() const;
	/** Is playback in reverse? */
	bool IsPlayingInReverse() const;

	/** Retrieve the delta used when dragging the spinbox. */
	double GetSpinboxDelta() const;

	/** Frame or seconds to display for text and input. */
	EFrameNumberDisplayFormats GetDisplayFormat() const;

	/** Sets the display format. */
	void SetDisplayFormat(EFrameNumberDisplayFormats InDisplayFormat);

	/** Validates the current display format is set to the given format. */
	bool CompareDisplayFormat(EFrameNumberDisplayFormats InDisplayFormat) const;

	/** Set the play rate (time dilation multiplier). */
	void SetPlayRate(float InPlayRate);

	/** Retrieve the text version of the display format. */
	FText GetDisplayFormatAsText() const;
	
	/** The frame rate, used for number inputs. */
	FFrameRate GetFrameRate() const;

	/** Convert raw seconds to frame time. */
	FQualifiedFrameTime SecondsToFrameTime(double InTime) const;

	/** The forward icon to use for the play forward button. */
	const FSlateBrush* GetPlayForwardIcon() const;
	/** The reverse icon to use for the play reverse button. */
	const FSlateBrush* GetPlayReverseIcon() const;
	/** The loop icon to use for the loop button. */
	const FSlateBrush* GetLoopIcon() const;
	/** The tooltip for the loop button. */
	FText GetLoopTooltip() const;

	/** Create the dropdown box for playback settings. */
	TSharedRef<SWidget> MakePlaybackSettingsDropdown();

	/** Make the menu content that allows picking a play rate. */
	TSharedRef<SWidget> MakePlayRateMenuContent();

	/** The widget used for displaying upgrade notifications. */
	TSharedRef<SWidget> MakeUpgradingWidget();

	/** Get the text displayed in the upgrade notification. */
	FText GetUpgradeText() const;

	/** The percent the upgrade has finished. */
	float GetUpgradePercent() const;

	/** Returns whether the playback widgets should be enabled or not by calling the IsPlaybackEnabledDelegate delegate. */
	bool IsPlaybackEnabled() const;

	/** Returns a FReply::Handled() and focuses the time slider. Useful so the user can click the play button but scrub using left and right keys. */
	FReply HandleAndFocusTimeSlider() const;
private:
	/** Delegate for pressing play forward. */
	FOnButtonPressed OnPlayForwardDelegate;
	
	/** Delegate for pressing play reverse. */
	FOnButtonPressed OnPlayReverseDelegate;

	/** Delegate for pressing go to first frame. */
	FOnButtonPressed OnFirstFrameDelegate;

	/** Delegate for pressing go to last frame. */
	FOnButtonPressed OnLastFrameDelegate;

	/** Delegate for going to the previous frame. */
	FOnButtonPressed OnPreviousFrameDelegate;
	
	/** Delegate for going to the next frame. */
	FOnButtonPressed OnNextFrameDelegate;

	/** Delegate for setting the play rate. */
	FOnSetFloat OnSetPlayRateDelegate;

	/** Delegate for exiting the current recording. */
	FOnButtonPressed OnExitPlaybackDelegate;

	/** Delegate for selecting a recording to play. */
	FOnButtonPressed OnSelectRecordingDelegate;

	/** Delegate checking if the recording playback is enabled */
	FOnGetChecked IsPlaybackEnabledDelegate;

	/** Delegate if recording is upgrading. */
	FOnGetChecked IsUpgradingDelegate;

	/** Delegate for upgrade percent. */
	FOnGetFloat OnGetUpgradePercentDelegate;
	
	/** Delegate for checking if the recording playback is paused. */
	FOnGetChecked OnGetPausedDelegate;
	
	/** Delegate for checking if the recording playback is playing in reverse. */
	FOnGetChecked OnGetIsInReverseDelegate;

	/** Delegate for checking if the recording playback should loop. */
	FOnGetChecked OnGetLoopingDelegate;
	
	/** Delegate to set the looping option. */
	FOnSetChecked OnSetLoopingDelegate;
	
	/** Delegate for getting the total length of the recording. */
	FOnGetTime OnGetTotalLengthDelegate;

	/** Delegate for checking the playhead time. */
	FOnGetTime OnGetCurrentTimeDelegate;
	
	/** Delegate for setting the current playhead time. */
	FOnSetTime OnSetCurrentTimeDelegate;

	/** Delegate for getting the view range. */
	FOnGetDoubleRange OnGetViewRangeDelegate;
	
	/** Delegate for setting the view range. */
	FOnSetDoubleRange OnSetViewRangeDelegate;

	/** Retrieve the buffered frame range. */
	FOnGetIntArrayRange OnGetFrameBufferRanges;

	/** Delegate for getting the selection start. */
	FOnGetTime OnGetSelectionStartTimeDelegate;

	/** Delegate for setting the selection start. */
	FOnSetTime OnSetSelectionStartTimeDelegate;

	/** Delegate for getting the selection end. */
	FOnGetTime OnGetSelectionEndTimeDelegate;

	/** Delegate for setting the selection start. */
	FOnSetTime OnSetSelectionEndTimeDelegate;

	/** Retrieve the frame rate. */
	FOnGetFrame OnGetFrameRate;

	/** Delegate triggered when scrubbing starts. */
	FSimpleDelegate OnScrubbingStartDelegate;

	/** Delegate triggered when scrubbing stops. */
	FSimpleDelegate OnScrubbingEndDelegate;

	/** The number interface for displaying the correct frame format. */
	TSharedPtr<FFrameNumberInterface> NumberInterface;

	/** The format to display/edit values in. */
	EFrameNumberDisplayFormats DisplayFormat = EFrameNumberDisplayFormats::NonDropFrameTimecode;

	/** Holds the LiveLinkHubTimeSlider widget. */
	TSharedPtr<SWidget> TimeSlider;

	/** Helper enum to track KeyDown state. */
	enum class EScrubDirection : uint8
	{
		Left,
		Right
	};

	/** Keeps track of how many frames since the left and right key have been pressed. */
	uint32 KeyDownCounter[2] = { 0, 0 };

	/** Keeps track of whether the left or right key is being pressed. */
	bool KeyDown[2] = { 0, 0 };

	/** Counter used to advance the recording while holding down left or right key. */
	uint8 KeyDownScrubCounter = 0;

	/** Cached value of the recording playback rate. */
	float CachedPlayRate = 1.0f;

	/** Number of frames to wait between advancing the frame while a key is held down.*/
	static constexpr uint8 FramesBetweenKeyDownScrub = 3;

	/** Minimum number of frames to consider a frame pressed as being held down for scrubbing (vs. tapping it for a single frame jump). */
	static constexpr uint8 MinFramesToBeginScrubbing = 30;
};