// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorFontGlyphs.h"
#include "Engine/DeveloperSettings.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Modules/ModuleManager.h"
#include "Settings/LiveLinkHubSettings.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubSettings"

/**
 * LiveLinkHub Settings widget.
 */
class SLiveLinkHubSettings : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLiveLinkHubSettings) {}
	SLATE_END_ARGS()

	/** ID for the settings tab. */
	const FName SettingsTabId = "LiveLinkHubSettings";

	/**
	 * Create the settings combo button.
	 */
	void Construct(const FArguments& InArgs)
	{
		UnregisterSettings();

		// Settings editor might not be loaded at this point.
		ISettingsEditorModule* SettingsEditorModule = static_cast<ISettingsEditorModule*>(FModuleManager::Get().LoadModule("SettingsEditor"));
		SettingsEditorModule->SetShouldRegisterSettingCallback(FShouldRegisterSettingsDelegate::CreateLambda([](UDeveloperSettings* Settings) { return false; }));

		// Unregister in case the widget was re-created.
		FGlobalTabmanager::Get()->UnregisterTabSpawner(SettingsTabId);

		FGlobalTabmanager::Get()->RegisterTabSpawner(SettingsTabId, FOnSpawnTab::CreateRaw(this, &SLiveLinkHubSettings::SpawnSettingsTab))
			.SetDisplayName(LOCTEXT("LiveLinkHubSettingsTabLabel", "Settings"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		const FToolBarStyle& ToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolbar");

		ChildSlot
			[
				SNew(SBox)
				.Padding(FMargin(0, 4.0f, 4.0f, 4.0f))
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(2.0, 4.0))
					.ButtonStyle(&ToolbarStyle.ButtonStyle)
					.ComboButtonStyle(&ToolbarStyle.ComboButtonStyle)
					.ForegroundColor(FSlateColor::UseStyle())
					.ToolTipText(LOCTEXT("SettingsToolTip", "Settings"))
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("FullBlueprintEditor.EditGlobalOptions"))
					]
					.OnGetMenuContent(this, &SLiveLinkHubSettings::OnSettingsComboBoxOpen)
				]
			];
	}

	~SLiveLinkHubSettings()
	{
		FGlobalTabmanager::Get()->UnregisterTabSpawner(SettingsTabId);
	}

	/** Unregister all settings except the ones that are relevant to the hub. */
	void UnregisterSettings()
	{
		ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

		TArray<FName> ContainerNames;
		SettingsModule.GetContainerNames(ContainerNames);

		const TArray<FName>& SettingsToDisplay = GetDefault<ULiveLinkHubSettings>()->ProjectSettingsToDisplay;

		for (FName ContainerName : ContainerNames)
		{
			if (TSharedPtr<ISettingsContainer> Container = SettingsModule.GetContainer(ContainerName))
			{
				TArray<TSharedPtr<ISettingsCategory>> Categories;
				Container->GetCategories(Categories);

				for (TSharedPtr<ISettingsCategory> SettingsCategory : Categories)
				{
					TArray<TSharedPtr<ISettingsSection>> Sections;
					constexpr bool bIgnoreVisibility = true;
					SettingsCategory->GetSections(Sections, bIgnoreVisibility);

					for (TSharedPtr<ISettingsSection> Section : Sections)
					{
						if (!SettingsToDisplay.Contains(Section->GetName()))
						{
							SettingsModule.UnregisterSettings(Container->GetName(), SettingsCategory->GetName(), Section->GetName());
						}
					}
				}
			}
		}
	}

	/** Handler called when the combo button is clicked. */
	TSharedRef<SWidget> OnSettingsComboBoxOpen() const
	{
		const FUIAction OpenHubSettingsAction(
			FExecuteAction::CreateLambda([TabId = SettingsTabId]
				{
					if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
					{
						FGlobalTabmanager::Get()->TryInvokeTab(TabId);
					}
				}), FCanExecuteAction(), FIsActionChecked());

		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, NULL);

		MenuBuilder.BeginSection("HubSettings", LOCTEXT("HubSettingsLabel", "Hub Settings"));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SettingsMenuEntry", "Settings..."),
			LOCTEXT("SettingsMenuEntry_ToolTip", "Open the Settings tab."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"),
			OpenHubSettingsAction,
			NAME_None,
			EUserInterfaceActionType::Button);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PluginsMenuEntry", "Plugins..."),
			LOCTEXT("PluginsMenuEntry_ToolTip", "Open the Plugins Browser tab."),
			FSlateIcon("PluginStyle", "Plugins.TabIcon"),
			FExecuteAction::CreateStatic([]()
				{
					FGlobalTabmanager::Get()->TryInvokeTab(FName("PluginsEditor"));
				}));
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	/** Creates the settings viewer tab. */
	TSharedRef<SDockTab> SpawnSettingsTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<SWidget> SettingsEditor = SNullWidget::NullWidget;
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer("Project");

			if (SettingsContainer.IsValid())
			{
				ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");

				SettingsEditorModel = SettingsEditorModule.CreateModel(SettingsContainer.ToSharedRef());
				SettingsEditor = SettingsEditorModule.CreateEditor(SettingsEditorModel.ToSharedRef());
			}
		}

		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SettingsEditor
			];
	}

private:
	/** Holds the view state for the settings. */
	TSharedPtr<ISettingsEditorModel> SettingsEditorModel;
};

#undef LOCTEXT_NAMESPACE /*LiveLinkHubSettings*/
