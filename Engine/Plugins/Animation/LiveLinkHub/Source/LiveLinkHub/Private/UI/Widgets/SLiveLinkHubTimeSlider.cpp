// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkHubTimeSlider.h"

void SLiveLinkHubTimeSlider::Construct(const FArguments& InArgs)
{
	BufferRanges = (InArgs._BufferRange);

	SSimpleTimeSlider::Construct(InArgs._BaseArgs);
}

int32 SLiveLinkHubTimeSlider::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 Result = SSimpleTimeSlider::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	OnPaintExtendedSlider(MirrorLabels.Get(), AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return Result;
}

void SLiveLinkHubTimeSlider::OnPaintExtendedSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = ViewRange.Get();
	const float LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
	const float LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
	const float LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;
	
	if (LocalSequenceLength > 0)
	{
		FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.GetLocalSize());

		for (const TRange<double>& BufferRange : BufferRanges.Get())
		{
			float LeftBuffer = RangeToScreen.InputToLocalX(BufferRange.GetLowerBoundValue());
			float RightBuffer = RangeToScreen.InputToLocalX(BufferRange.GetUpperBoundValue());
			float Height = AllottedGeometry.GetLocalSize().Y * ClampRangeHighlightSize.Get();
	
			FPaintGeometry RangeGeometry;
			if (bMirrorLabels)
			{
				RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(RightBuffer-LeftBuffer, Height), FSlateLayoutTransform(FVector2f(LeftBuffer, 0)));
			}
			else
			{
				RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(RightBuffer-LeftBuffer, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(LeftBuffer, AllottedGeometry.GetLocalSize().Y - Height / 3.f)));
			}

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				RangeGeometry,
				CursorBackground,
				DrawEffects,
				FLinearColor::White
				);
		}
	}
}
