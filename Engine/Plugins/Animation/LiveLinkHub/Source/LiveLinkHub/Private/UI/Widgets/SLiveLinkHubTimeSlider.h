// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Recording/LiveLinkRecordingRangeHelpers.h"
#include "SSimpleTimeSlider.h"

/**
 * Live link hub slider for scrubbing playback and reporting the playhead position.
 * Additionally, it supports displaying currently buffered frames.
 */
class SLiveLinkHubTimeSlider : public SSimpleTimeSlider
{
public:

	SLATE_BEGIN_ARGS(SLiveLinkHubTimeSlider)
	{}
		/* The buffered frame size. */
		SLATE_ATTRIBUTE(UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<double>, BufferRange)
		/** SimpleTimeSlider base args. */
		SLATE_ARGUMENT(SSimpleTimeSlider::FArguments, BaseArgs)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

protected:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Paint live link hub specific slider details. */
	void OnPaintExtendedSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	
private:
	/** The frame buffer range to render. */
	TAttribute<UE::LiveLinkHub::RangeHelpers::Private::TRangeArray<double>> BufferRanges;
};