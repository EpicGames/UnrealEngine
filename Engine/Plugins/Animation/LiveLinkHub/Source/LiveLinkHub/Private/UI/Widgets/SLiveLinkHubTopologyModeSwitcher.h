// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

#include "LiveLinkHub.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubTopologyModeSwitcher"

class SLiveLinkHubTopologyModeSwitcher : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkHubTopologyModeSwitcher)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get();

		const bool bReadOnly = !LiveLinkHub->CanSetTopologyMode();

		const ELiveLinkTopologyMode Mode = LiveLinkHub->GetTopologyMode();

		FText ModeTooltip = GetModeTooltip(Mode);

		if (bReadOnly)
		{
			ModeTooltip = FText::Format(LOCTEXT("ReadOnlyToolTip", "{0}\nNote: Since the mode was set by a command line, it cannot be changed at runtime."), ModeTooltip);
		}

		HubModeIcon = FSlateStyleRegistry::FindSlateStyle("LiveLinkStyle")->GetBrush("LiveLinkHub.HubMode");
		SpokeModeIcon = FSlateStyleRegistry::FindSlateStyle("LiveLinkStyle")->GetBrush("LiveLinkHub.SpokeMode");

		const FToolBarStyle& ToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolbar");

		const bool bCanSetTopologyMode = FLiveLinkHub::Get()->CanSetTopologyMode();

		ChildSlot
		[
			SAssignNew(ModeButton, SComboButton)
			.ContentPadding(FMargin(0.0))
			.ButtonStyle(&ToolbarStyle.ButtonStyle)
			.ComboButtonStyle(&ToolbarStyle.ComboButtonStyle)
			.ForegroundColor(FSlateColor::UseStyle())
			.ToolTipText(ModeTooltip)
			.IsEnabled(bCanSetTopologyMode)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D{16, 16})
					.Image(this, &SLiveLinkHubTopologyModeSwitcher::GetModeIcon)
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0, 0.0, 0.0, 0.0))
				.AutoWidth()
				[
					SAssignNew(ButtonContent, STextBlock)
					.Text(UEnum::GetDisplayValueAsText(Mode))
				]
			]
			.OnGetMenuContent(this, &SLiveLinkHubTopologyModeSwitcher::OnGetMenuContent)
		];

		LiveLinkHub->OnTopologyModeChanged().AddSP(this, &SLiveLinkHubTopologyModeSwitcher::OnModeChanged);
	}

	virtual ~SLiveLinkHubTopologyModeSwitcher()
	{
		if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
		{
			LiveLinkHub->OnTopologyModeChanged().RemoveAll(this);
		}
	}

private:
	/** Handles toggling the topology mode. */
	FReply OnModeClicked()
	{
		FLiveLinkHub::Get()->ToggleTopologyMode();
		return FReply::Handled();
	}

	/** Handles updating the button's text and tooltip when the topology mode changed. */
	void OnModeChanged(ELiveLinkTopologyMode Mode)
	{
		UEnum* Enum = StaticEnum<ELiveLinkTopologyMode>();

		ELiveLinkTopologyMode OtherMode = ELiveLinkTopologyMode::Spoke;
		if (Mode == ELiveLinkTopologyMode::Spoke)
		{
			OtherMode = ELiveLinkTopologyMode::Hub;
		}
		
		ButtonContent->SetText(UEnum::GetDisplayValueAsText(Mode));
		ModeButton->SetToolTipText(FText::Format(LOCTEXT("ButtonContentToolTip", "{0}\nClick to change to {1} mode"), GetModeTooltip(Mode), UEnum::GetDisplayValueAsText(OtherMode)));
	}

	const FSlateBrush* GetModeIcon() const
	{
		const ELiveLinkTopologyMode Mode =  FLiveLinkHub::Get()->GetTopologyMode();
		if (Mode == ELiveLinkTopologyMode::Spoke)
		{
			return SpokeModeIcon;
		}
		else
		{
			return HubModeIcon;
		}
	}

	TSharedRef<SWidget> OnGetMenuContent() const
	{

		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, NULL);

		MenuBuilder.BeginSection("OperationMode", LOCTEXT("OperationModeLabel", "Operation Mode"));

		const bool bCanSetSpokeMode = FLiveLinkHub::Get()->GetTopologyMode() != ELiveLinkTopologyMode::Spoke;
		const bool bCanSetHubMode = FLiveLinkHub::Get()->GetTopologyMode() != ELiveLinkTopologyMode::Hub;

		FExecuteAction ToggleModeAction = FExecuteAction::CreateLambda([]()
			{
				FLiveLinkHub::Get()->ToggleTopologyMode();
			});

		FUIAction SpokeAction;
		SpokeAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanSetSpokeMode]() { return bCanSetSpokeMode; });
		SpokeAction.ExecuteAction = ToggleModeAction;

		FUIAction HubAction;
		HubAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanSetHubMode]() { return bCanSetHubMode; });
		HubAction.ExecuteAction = ToggleModeAction;


		MenuBuilder.AddMenuEntry(
			LOCTEXT("SpokeModeEntry", "Spoke"),
			GetModeTooltip(ELiveLinkTopologyMode::Spoke),
			FSlateIcon("LiveLinkStyle", "LiveLinkHub.SpokeMode"),
			MoveTemp(SpokeAction),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("HubModeEntry", "Hub"),
			GetModeTooltip(ELiveLinkTopologyMode::Hub),
			FSlateIcon("LiveLinkStyle", "LiveLinkHub.HubMode"),
			MoveTemp(HubAction),
			NAME_None,
			EUserInterfaceActionType::Button);
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	/** Returns the tooltip for a mode. */
	static FText GetModeTooltip(ELiveLinkTopologyMode InMode)
	{
		UEnum* Enum = StaticEnum<ELiveLinkTopologyMode>();
		const int32 ModeEnumIndex = Enum->GetIndexByValue(static_cast<int64>(InMode));
		return Enum->GetToolTipTextByIndex(ModeEnumIndex);
	}

private:
	/** Button that's responsible for switching the topology mode of the app. */
	TSharedPtr<SComboButton> ModeButton;
	/** Text indicating the current mode. */
	TSharedPtr<STextBlock> ButtonContent;
	/** Icon for Hub mode. */
	const FSlateBrush* HubModeIcon = nullptr;
	/** Icon for Spoke mode. */
	const FSlateBrush* SpokeModeIcon = nullptr;
};

#undef LOCTEXT_NAMESPACE