// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkRecordingSessionInfo.h"

#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "SLiveLinkRecordingSessionInfo"


void SLiveLinkRecordingSessionInfo::Construct(const FArguments& InArgs)
{
	const FMargin IntraElementPadding = FMargin(6.0, 4.0);

	ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Margin(IntraElementPadding)
			.Text(LOCTEXT("SessionEditLabel", "Session"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(60)
			.Padding(IntraElementPadding)
			.Text_Lambda([&SessionInfo] { return FText::FromString(SessionInfo.GetSessionName()); })
			.OnTextCommitted_Lambda([&SessionInfo]
				(const FText& InText, ETextCommit::Type)
				{
					SessionInfo.SetSessionName(InText.ToString());
				})
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Margin(IntraElementPadding)
			.Text(LOCTEXT("SlateEditLabel", "Slate"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SEditableTextBox)
			.Padding(IntraElementPadding)
			.MinDesiredWidth(60)
			.Text_Lambda([&SessionInfo] { return FText::FromString(SessionInfo.GetSlateName()); })
			.OnTextCommitted_Lambda([&SessionInfo]
				(const FText& InText, ETextCommit::Type)
				{
					SessionInfo.SetSlateName(InText.ToString());
				})
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Margin(IntraElementPadding)
			.Text(LOCTEXT("TakeEditLabel", "Take"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(20)
			.Padding(IntraElementPadding)
			.Text_Lambda([&SessionInfo] { return FText::FromString(LexToString(SessionInfo.GetTakeNumber())); })
			.OnTextCommitted_Lambda(
				[&SessionInfo]
				(const FText& InText, ETextCommit::Type)
				{
					int32 TakeNumber;
					LexFromString(TakeNumber, *InText.ToString());
					SessionInfo.SetTakeNumber(TakeNumber);
				})
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(IntraElementPadding)
		[
			SNew(SPositiveActionButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.ToolTipText(LOCTEXT("IncrementTake_ToolTip", "Increment the current take number"))
			.OnClicked_Lambda(
				[&SessionInfo]
				() -> FReply
				{
					SessionInfo.SetTakeNumber(SessionInfo.GetTakeNumber() + 1);
					return FReply::Handled();
				})
		]
	];
}


#undef LOCTEXT_NAMESPACE
