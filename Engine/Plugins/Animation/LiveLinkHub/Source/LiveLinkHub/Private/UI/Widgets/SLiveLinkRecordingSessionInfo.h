// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"


/** Display and edit the session name, slate name, take number. */
class SLiveLinkRecordingSessionInfo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkRecordingSessionInfo) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
