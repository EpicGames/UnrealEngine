// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkTimecode.h"

#include "EditorFontGlyphs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ILiveLinkModule.h"
#include "Internationalization/Text.h"
#include "ILiveLinkModule.h"
#include "ISettingsModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/STimecode.h"

#include "LiveLinkHubModule.h"
#include "Session/LiveLinkHubSessionManager.h"

#define LOCTEXT_NAMESPACE "LiveLinkHub"

FSlateColor SLiveLinkTimecode::GetIconColor() const
{
	const ULiveLinkHubTimeAndSyncSettings* TimeAndSyncSettings = GetDefault<ULiveLinkHubTimeAndSyncSettings>();
	const bool bCustomTimeStepEnabled = TimeAndSyncSettings->bUseLiveLinkHubAsCustomTimeStepSource;

	TSharedPtr<FSlateStyleSet> Style = ILiveLinkModule::Get().GetStyle();
	if (bCustomTimeStepEnabled)
	{
		const UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		const bool bHasCustomTimeStep = Profile && Profile->GetCustomTimeStep() != nullptr;
		return bHasCustomTimeStep ? Style->GetSlateColor("LiveLink.Color.Valid") : Style->GetSlateColor("LiveLink.Color.Error");
	}
	return FSlateColor(FLinearColor::Gray);
}

FSlateColor SLiveLinkTimecode::GetStatusColor() const
{
	const ULiveLinkHubTimeAndSyncSettings* TimeAndSyncSettings = GetDefault<ULiveLinkHubTimeAndSyncSettings>();
	const bool bTimecodeSourceEnabled = TimeAndSyncSettings->bUseLiveLinkHubAsTimecodeSource;

	TSharedPtr<FSlateStyleSet> Style = ILiveLinkModule::Get().GetStyle();

	if (bTimecodeSourceEnabled)
	{
		const UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		const bool bHasTimecodeProvider = Profile && Profile->GetTimecodeProvider() != nullptr;
		return bHasTimecodeProvider ? Style->GetSlateColor("LiveLink.Color.Valid") : Style->GetSlateColor("LiveLink.Color.Error");
	}

	return FSlateColor(FLinearColor::Gray);
}

FText SLiveLinkTimecode::GetTimecodeTooltip() const
{
	const ULiveLinkHubTimeAndSyncSettings* TimeAndSyncSettings = GetDefault<ULiveLinkHubTimeAndSyncSettings>();
	const bool bTimecodeSourceEnabled = TimeAndSyncSettings->bUseLiveLinkHubAsTimecodeSource;
	const bool bCustomTimeStepEnabled = TimeAndSyncSettings->bUseLiveLinkHubAsCustomTimeStepSource;

	if (bCustomTimeStepEnabled)
	{
		const UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		const bool bHasCustomTimeStep = Profile && Profile->GetCustomTimeStep() != nullptr;
		if (!bHasCustomTimeStep)
		{
			return LOCTEXT("LiveLinkTimeCode_CustomTimeStepError", "Connected clients will not be synchronized unless you select both a framerate and a subject for the Custom Time Step.");
		}
	}

	if (bTimecodeSourceEnabled && !bCustomTimeStepEnabled)
	{
		return LOCTEXT("LiveLinkTimeCode_TimecodeConnected", "Sending Timecode data to connected editors.");
	}
	else if (!bTimecodeSourceEnabled && bCustomTimeStepEnabled)
	{
		return LOCTEXT("LiveLinkTimeCode_CustomTimeStepConnected", "Sending CustomTimestep data to connected editors.");
	}
	else if (bTimecodeSourceEnabled && bCustomTimeStepEnabled)
	{
		return LOCTEXT("LiveLinkTimeCode_TimecodeAndCustomTimeStepConnected", "Sending Timecode and CustomTimeStep data to connected editors.");
	}

	return LOCTEXT("LiveLinkTimeCode_NotConnected", "No Timecode or CustomTimeStep data shared with connected editors.");
}

void SLiveLinkTimecode::Construct(const FArguments& InArgs)
{
	const FLiveLinkHubModule& LiveLinkHubModule = FModuleManager::Get().GetModuleChecked<FLiveLinkHubModule>("LiveLinkHub");
	const ISlateStyle* StyleSet = FSlateStyleRegistry::FindSlateStyle("LiveLinkStyle");
	check(StyleSet);

	ChildSlot
	[
		SNew(SHorizontalBox)
		.ToolTipText(this, &SLiveLinkTimecode::GetTimecodeTooltip)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(5, 0, 3, 0)
		[
			SNew(SImage)
			.Image(FSlateIcon("LiveLinkStyle", "LiveLinkHub.TimecodeGenlock").GetIcon())
			.ColorAndOpacity(this, &SLiveLinkTimecode::GetIconColor)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(2, 0))
		[
			SNew(STimecode)
			.DisplayLabel(false)
			.TimecodeFont(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
			.LabelColor(this, &SLiveLinkTimecode::GetStatusColor)
			.TimecodeColor(this, &SLiveLinkTimecode::GetStatusColor)
			.Timecode(MakeAttributeLambda([]
			{
				return FApp::GetTimecode();
			}))
			.bDisplaySubframes(false)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(2, 2, 0, 2))
		[
			SNew(STextBlock)
			.Font(FCoreStyle::Get().GetFontStyle(TEXT("SmallText")))
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(MakeAttributeLambda([] {return FApp::GetTimecodeFrameRate().ToPrettyText();}))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ContentPadding(0.0)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolbar").ButtonStyle)
			.OnClicked(this, &SLiveLinkTimecode::OnClickOpenSettings)
			.ToolTipText(LOCTEXT("TimeAndSyncToolTip", "Open Time and Sync settings"))
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(StyleSet->GetBrush("LiveLinkHub.EllipsisIcon"))
			]
		]
	];
}

FReply SLiveLinkTimecode::OnClickOpenSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->ShowViewer("Project", "Application", "Timing & Sync");
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
