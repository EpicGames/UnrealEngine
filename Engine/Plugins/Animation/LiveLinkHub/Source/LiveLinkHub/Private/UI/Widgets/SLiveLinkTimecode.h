// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/NameTypes.h"

/**
 * Utility class to display timecode status on the menu bar. Provides user with
 * a drop down of supported time code values including subjects that are in the
 * Live Link session.
 */
class SLiveLinkTimecode : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLiveLinkTimecode) {}
		/** The tab's content */
	SLATE_END_ARGS()

	/**
	 * @param InArgs
	 */
	void Construct(const FArguments& InArgs);

private:
	/** Returns the color of the icon depending on whether timecode and genlock is set up correctly. */
	FSlateColor GetIconColor() const;

	/** Indicate the status of the time code send status */
	FSlateColor GetStatusColor() const;

	/** Gets the tooltip text for the time code widget */
	FText GetTimecodeTooltip() const;

	/** Open project settings to the Time & Sync section. */
	FReply OnClickOpenSettings() const;
};
