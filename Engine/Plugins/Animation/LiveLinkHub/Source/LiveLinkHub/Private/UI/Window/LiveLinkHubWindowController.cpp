// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubWindowController.h"

#include "Clients/LiveLinkHubProvider.h"
#include "CoreGlobals.h"
#include "Framework/Application/SWindowTitleBar.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "LiveLinkHub.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "UI/Widgets/SLiveLinkHubMemoryStats.h"
#include "UI/Widgets/SLiveLinkHubTopologyModeSwitcher.h"
#include "UI/Window/ModalWindowManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubWindowController"

const FName FLiveLinkHubWindowController::LiveLinkHubTabId = "LiveLinkHub";

FLiveLinkHubWindowController::FLiveLinkHubWindowController()
{
	IModularFeatures::Get().RegisterModularFeature(IEditorMainFrameProvider::GetModularFeatureName(), this);

	const FName LayoutName = TEXT("LiveLinkHub_v1.2");
	DefaultLayout = FTabManager::NewLayout(LayoutName)
		->AddArea
		(
			// Toolkits window
			FTabManager::NewPrimaryArea()
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->AddTab(LiveLinkHubTabId, ETabState::ClosedTab)
				->AddTab("StandaloneToolkit", ETabState::ClosedTab) // This tab is placed here in case the user opens an asset editor (ie. Media Profile)
			)
		);
}

FLiveLinkHubWindowController::~FLiveLinkHubWindowController()
{
	if (RootWindow)
	{
		RootWindow->SetOnWindowClosed(nullptr);
	}

	IModularFeatures::Get().UnregisterModularFeature(IEditorMainFrameProvider::GetModularFeatureName(), this);
}

TSharedRef<SWidget> FLiveLinkHubWindowController::CreateMainFrameContentWidget() const
{
	// We defer creation until RestoreLayout is called so that the widget has a valid RootWindow to use for restoring layout.
	return SNullWidget::NullWidget;
}

FMainFrameWindowOverrides FLiveLinkHubWindowController::GetDesiredWindowConfiguration() const
{
	FMainFrameWindowOverrides Overrides;
	Overrides.WindowSize = FVector2D(1200.0f, 800.0f);
	Overrides.WindowTitle = LOCTEXT("WindowTitle", "Live Link Hub");
	Overrides.bEmbedTitleAreaContent = true;
	Overrides.bInitiallyMaximized = false;
	return Overrides;
}

void FLiveLinkHubWindowController::RestoreLayout(TSharedPtr<FAssetEditorToolkit> AssetEditorToolkit)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHubWindowController::RestoreLayout);

	FModuleManager::Get().LoadModuleChecked<IMainFrameModule>("MainFrame").CreateDefaultMainFrame(false, false);

	RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	RootWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateRaw(this, &FLiveLinkHubWindowController::CloseRootWindowOverride));
	RootWindow->SetOnWindowClosed(FOnWindowClosed::CreateRaw(this, &FLiveLinkHubWindowController::OnWindowClosed));

	constexpr bool bEmbedTitleAreaContent = true;
	const TSharedPtr<SWidget> Content = FGlobalTabmanager::Get()->RestoreFrom(DefaultLayout.ToSharedRef(), RootWindow, bEmbedTitleAreaContent, EOutputCanBeNullptr::Never);
	RootWindow->SetContent(Content.ToSharedRef());

	// Pass a dummy object to the asset editor since we're not actually editing an object.
	UObject* DummyObject = GetMutableDefault<UObject>();
	AssetEditorToolkit->InitAssetEditor(EToolkitMode::Standalone, nullptr, LiveLinkHubTabId, DefaultLayout.ToSharedRef(), /*bCreateDefaultStandaloneMenu*/true, /*bCreateDefaultToolbar*/true, DummyObject, /*bInIsToolbarFocusable*/true, true, {});

	TSharedRef<SWidget> ProviderNameWidget =
		SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Text(FText::FromString(FLiveLinkHub::Get()->GetLiveLinkProvider()->GetProviderName()))
		];

	TSharedRef<SWidget> RightMenuWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ProviderNameWidget
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f)
		[
			SNew(SLiveLinkHubMemoryStats)
		];

	FGlobalTabmanager::Get()->SetMainTab(LiveLinkHubTabId);
	TSharedPtr<SDockTab> MainTab = FGlobalTabmanager::Get()->FindExistingLiveTab(LiveLinkHubTabId);
	MainTab->SetTitleBarRightContent(RightMenuWidget);
	MainTab->SetTabIcon(FSlateIcon("LiveLinkStyle", "LiveLinkHub.Icon.Small").GetIcon());
}

void FLiveLinkHubWindowController::CloseRootWindowOverride(const TSharedRef<SWindow>& Window)
{
	if (GetDefault<ULiveLinkHubSettings>()->bConfirmClose)
	{
		const EAppReturnType::Type Response = FMessageDialog::Open(
			EAppMsgCategory::Info,
			EAppMsgType::YesNo,
			LOCTEXT("ConfirmClose", "Are you sure you want to close Live Link Hub?"),
			LOCTEXT("ConfirmCloseTitle", "Close Live Link Hub")
		);

		bool bOkToExit = (Response == EAppReturnType::Yes);
		if (!bOkToExit)
		{
			return;
		}
	}

	RootWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride());
	RootWindow->RequestDestroyWindow();
}

void FLiveLinkHubWindowController::OnWindowClosed(const TSharedRef<SWindow>& Window)
{
	if (TSharedPtr<FLiveLinkHub> LiveLinkHub = FLiveLinkHub::Get())
	{
		FGlobalTabmanager::Get()->SaveAllVisualState();
		LiveLinkHub->CloseWindow(EAssetEditorCloseReason::CloseAllAssetEditors);
	}

	RootWindow.Reset();

	RequestEngineExit(TEXT("FLiveLinkHubWindowController::OnWindowClosed"));
}


#undef LOCTEXT_NAMESPACE 
