// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IEditorMainFrameProvider.h"

#include "Framework/Docking/TabManager.h"

class FAssetEditorToolkit;
class SWindow;

/** Responsible for creating the Slate window for the hub. */
class FLiveLinkHubWindowController : public TSharedFromThis<FLiveLinkHubWindowController>, public IEditorMainFrameProvider
{
public:
	FLiveLinkHubWindowController();
	~FLiveLinkHubWindowController();

	/** Get the root window. */
	TSharedPtr<SWindow> GetRootWindow() const { return RootWindow; }
	/** Restore the window's layout from a config. */
	void RestoreLayout(TSharedPtr<FAssetEditorToolkit> AssetEditorToolkit);

	//~ Begin IEditorMainFrameProvider
	virtual bool IsRequestingMainFrameControl() const { return true; }
	virtual FMainFrameWindowOverrides GetDesiredWindowConfiguration() const;
	virtual TSharedRef<SWidget> CreateMainFrameContentWidget() const;
	//~ End IEditorMainFrameProvider

private:
	/** Create the main window. */
	TSharedRef<SWindow> CreateWindow();
	/** Override to handle confirming if the user wants to quit. */
	void CloseRootWindowOverride(const TSharedRef<SWindow>& Window);
	/** Window closed handler. */
	void OnWindowClosed(const TSharedRef<SWindow>& Window);
private:
	/** The main window being managed */
	TSharedPtr<SWindow> RootWindow;
	/** Default layout for the app. */
	TSharedPtr<FTabManager::FLayout> DefaultLayout;
	/** Tab ID of the main LiveLinkHub  */
	static const FName LiveLinkHubTabId;
};
