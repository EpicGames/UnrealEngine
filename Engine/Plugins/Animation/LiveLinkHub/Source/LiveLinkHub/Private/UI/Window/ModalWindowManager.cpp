// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModalWindowManager.h"

#include "Dialog/SCustomDialog.h"

FModalWindowManager::FModalWindowManager(const TSharedRef<SWindow>& InMainWindow)
{
	MainWindow = InMainWindow;
	InMainWindow->GetOnWindowActivatedEvent().AddRaw(this, &FModalWindowManager::OnMainWindowActivated);

	// Fake modal windows should be closed when the root window is closed
	InMainWindow->GetOnWindowClosedEvent().AddRaw(this, &FModalWindowManager::OnWindowClosed);
}

FModalWindowManager::~FModalWindowManager()
{
	if (TSharedPtr<SWindow> MainWindowPin = MainWindow.Pin())
	{
		MainWindowPin->GetOnWindowClosedEvent().RemoveAll(this);
		MainWindowPin->GetOnWindowActivatedEvent().RemoveAll(this);
	}
}

void FModalWindowManager::ShowFakeModalWindow(const TSharedRef<SCustomDialog>& Dialog)
{
	if (ensureMsgf(bRootWindowIsValid, TEXT("Root window has already been destroyed!")))
	{
		Dialog->Show();
		PushWindow(Dialog);
	}
}

void FModalWindowManager::PushWindow(const TSharedRef<SWindow>& Window)
{
	if (ensureMsgf(bRootWindowIsValid, TEXT("Root window has already been destroyed!")))
	{
		check(!WindowStack.Contains(Window));
		WindowStack.Push(Window);
	
		Window->GetOnWindowClosedEvent().AddLambda([this](const TSharedRef<SWindow>& Window)
		{
			// Might request window destruction twice
			if (WindowStack.Contains(Window))
			{
				PopWindow();
			}
		});
	}
}

void FModalWindowManager::PopWindow()
{
	WindowStack.Pop();
}

void FModalWindowManager::OnMainWindowActivated()
{
	if (WindowStack.Num() > 0)
	{
		if (const TSharedPtr<SWindow> PinnedWindow = WindowStack.Top().Pin())
		{
			PinnedWindow->BringToFront();
			PinnedWindow->FlashWindow();
		}
	}
}

void FModalWindowManager::OnWindowClosed(const TSharedRef<SWindow>& InWindow)
{
	bRootWindowIsValid = false;

	for (TWeakPtr<SWindow> ModalWindow : TArray(WindowStack))
	{
		if (TSharedPtr<SWindow> Pin = ModalWindow.Pin())
		{
			Pin->RequestDestroyWindow();
		}
	}
	WindowStack.Empty();
}
