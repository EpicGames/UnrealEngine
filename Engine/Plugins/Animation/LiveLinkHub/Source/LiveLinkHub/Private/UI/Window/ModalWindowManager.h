// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SCustomDialog;
class SWindow;

/**
 * Manages "fake" modal windows because the hub cannot use "true" modal windows.
 *
 * True modal windows block the main thread which is not an option because the hub must remain responsive.
 */
class FModalWindowManager
{
public:

	FModalWindowManager(const TSharedRef<SWindow>& InMainWindow);
	~FModalWindowManager();

	/** Show a modal window and add it to our window stack. */
	void ShowFakeModalWindow(const TSharedRef<SCustomDialog>& Dialog);
	/** Called when a new fake modal window should be tracked on top of all windows. */
	void PushWindow(const TSharedRef<SWindow>& Window);
	/** Called when a fake modal window should no longer be tracked, i.e. when it is closed. */
	void PopWindow();

private:
	/** Holds the modal windows. */
	TArray<TWeakPtr<SWindow>> WindowStack;

	/** Modal windows can only be added when this is true */
	bool bRootWindowIsValid = true;

	/** Handler called when the main window is activated. */
	void OnMainWindowActivated();

	/** Handler called when the main window is closed, used for destroying all windows in the stack. */
	void OnWindowClosed(const TSharedRef<SWindow>& InWindow);

private:
	/** Weak pointer to the main window of the app. */
	TWeakPtr<SWindow> MainWindow;
};

