// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Containers/ObservableArray.h"
#include "Features/IModularFeatures.h"
#include "LiveLinkMessages.h"

#include "ILiveLinkHubClientsModel.generated.h"


struct FLiveLinkHubAuxChannelRequestMessage;
struct FLiveLinkClientInfoMessage;
struct FLiveLinkHubUEClientInfo;
struct FLiveLinkSubjectKey;
class FMessageEndpoint;

/** Information about a discovered client. */
struct FLiveLinkHubDiscoveredClientInfo
{
	/** Topology of the Client. */
	ELiveLinkTopologyMode TopologyMode = ELiveLinkTopologyMode::UnrealClient;
	/** IP of the client. */
	FString IP;
	/** Hostname of the client. */
	FString Hostname;
	/** Current level. */
	FString LevelName;
	/** Project that the client is in. */
	FString ProjectName;
};

/**
 * Identifier for a UE client receiving data from the hub.
 */
USTRUCT()
struct FLiveLinkHubClientId
{
	GENERATED_BODY()

	/** Default constructor, should only be used by the reflection code. */
	FLiveLinkHubClientId() = default;

	static FLiveLinkHubClientId NewId()
	{
		FLiveLinkHubClientId Id{};
		Id.Guid = FGuid::NewGuid();
		return Id;
	}

	bool IsValid() const
	{
		return Guid.IsValid();
	}

	/** Get the hash of this ID. */
	friend uint32 GetTypeHash(FLiveLinkHubClientId Id)
	{
		return GetTypeHash(Id.Guid);
	}

	bool operator==(const FLiveLinkHubClientId& Other) const
	{
		return Guid == Other.Guid;
	}

	FString ToString() const
	{
		return Guid.ToString();
	}

private:
	/** Unique identifier for this client. */
	UPROPERTY()
	FGuid Guid;
};


/** Exposes queries and operations for Unreal Editor clients connected to the local LLH instance. */
class ILiveLinkHubClientsModel : public IModularFeature
{
public:
	/** Types of client updates. */
	enum class EClientEventType
	{
		Discovered, // UE client was discovered by the hub.
		Disconnected, // Connection was lost with a client.
		Connected, // Client goes from Discovered -> Connected
		Reestablished, // Connection was restored with a previously disconnected client.
		Modified // Information about a given client has changed (ie. level has changed).
	};

	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnClientEvent, FLiveLinkHubClientId, EClientEventType);

	/** Name of the modular feature. */
	static FName GetModularFeatureName()
	{
		static const FName ModularFeatureName("LiveLinkHubClientsModel");
		return ModularFeatureName;
	}

	/** Convenience accessor for modular feature implementation. */
	static ILiveLinkHubClientsModel* Get()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const TArray<ILiveLinkHubClientsModel*> FeatureImplementations = 
			ModularFeatures.GetModularFeatureImplementations<ILiveLinkHubClientsModel>(GetModularFeatureName());
		ensure(FeatureImplementations.Num() <= 1);
		return FeatureImplementations.Num() > 0 ? FeatureImplementations[0] : nullptr;
	}

	/** Convenience accessor for modular feature implementation. */
	static ILiveLinkHubClientsModel& GetChecked()
	{
		ILiveLinkHubClientsModel* MaybeFeature = Get();
		check(MaybeFeature);
		return *MaybeFeature;
	}

public:
	virtual ~ILiveLinkHubClientsModel() = default;

	/** Delegate used to get noticed about changes to the client list. */
	virtual FOnClientEvent& OnClientEvent() = 0;

	/** Get the status text of a client. */
	virtual FText GetClientStatus(FLiveLinkHubClientId Client) const = 0;

	/** Get the list of clients that should be displayed in the clients panel according to the current session config. */
	virtual TArray<FLiveLinkHubClientId> GetSessionClients() const = 0;

	/** Get the list of clients discovered by the hub. */
	virtual TArray<FLiveLinkHubClientId> GetDiscoveredClients() const = 0;

	/** Get information about a discovered client. */
	virtual TOptional<FLiveLinkHubDiscoveredClientInfo> GetDiscoveredClientInfo(FLiveLinkHubClientId InClient) const = 0;

	/** Get information about a given client given its address. */
	virtual TOptional<FLiveLinkHubUEClientInfo> GetClientInfo(FLiveLinkHubClientId InClient) const = 0;

	/** Get the name of a client. */
	virtual FText GetClientDisplayName(FLiveLinkHubClientId InClient) const = 0;

	/** Send a connection message to this discovered client. */
	virtual void ConnectTo(FLiveLinkHubClientId InClient) = 0;

	/** Get whether a client should receive livelink data. */
	virtual bool IsClientEnabled(FLiveLinkHubClientId InClient) const = 0;

	/** Get whether a client is connected to the hub. */
	virtual bool IsClientConnected(FLiveLinkHubClientId InClient) const = 0;

	/** Set whether a client should receive livelink data. */
	virtual void SetClientEnabled(FLiveLinkHubClientId InClient, bool bInEnable) = 0;

	/** Get whether a subject is enabled on a given client. */
	virtual bool IsSubjectEnabled(FLiveLinkHubClientId InClient, FName SubjectName) const = 0;

	/** Set whether a subject should receive livelink data. */
	virtual void SetSubjectEnabled(FLiveLinkHubClientId InClient, FName SubjectName, bool bInEnable) = 0;

	/** Request to open an auxiliary endpoint with a client. */
	virtual void RequestAuxiliaryChannel(
		FLiveLinkHubClientId InClientId,
		FMessageEndpoint& InAuxEndpoint,
		UScriptStruct* InRequestTypeInfo,
		FLiveLinkHubAuxChannelRequestMessage& InRequest
	) = 0;

	/** Request to open an auxiliary endpoint with a client. */
	template<typename RequestType UE_REQUIRES(std::is_base_of_v<FLiveLinkHubAuxChannelRequestMessage, RequestType>)>
	void RequestAuxiliaryChannel(FLiveLinkHubClientId InClientId, FMessageEndpoint& InAuxEndpoint, RequestType& InRequest)
	{
		RequestAuxiliaryChannel(InClientId, InAuxEndpoint, RequestType::StaticStruct(), InRequest);
	}
};
