// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Framework/Docking/TabManager.h"
#include "Templates/SharedPointer.h"

class SWindow;

struct UE_DEPRECATED(5.6, "Customize LiveLinkHub by adding layouts instead.") LIVELINKHUB_API FLiveLinkHubComponentInitParams
{
	/** LiveLinkHub's main Window. */
	TSharedRef<SWindow> Window;

	/** The root area of the main window layout. Add tabs to this. */
	TSharedRef<FTabManager::FStack> MainStack;

	FLiveLinkHubComponentInitParams(TSharedRef<SWindow> InWindow, TSharedRef<FTabManager::FStack> InMainStack)
		: Window(MoveTemp(InWindow))
		, MainStack(MoveTemp(InMainStack))
	{}
};


/** Provides an interface for elements that wish to add additional content (tabs) to LiveLinkHub. */
class UE_DEPRECATED(5.6, "Customize LiveLinkHub by adding layouts instead.") LIVELINKHUB_API ILiveLinkHubComponent : public IModularFeature
{
public:
	/** Name of the modular feature. */
	static const FName ModularFeatureName;

	virtual ~ILiveLinkHubComponent() = default;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Initialises the component, e.g. registering tab spawners, etc. */
	virtual void Init(const FLiveLinkHubComponentInitParams& Params) = 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
