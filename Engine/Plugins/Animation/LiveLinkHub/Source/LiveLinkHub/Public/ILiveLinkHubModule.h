// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Modules/ModuleInterface.h"

class ILiveLinkHubModule : public IModuleInterface
{
public:
	/** Retrieve the namespace for Live Link Hub naming tokens. */
	static FString GetLiveLinkHubNamingTokensNamespace()
	{
		return TEXT("llh");
	}
};
