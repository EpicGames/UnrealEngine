// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

namespace UE::LiveLinkHub
{
	static const FName LiveLinkStyleName = "LiveLinkStyle";

	//~ Tab names to allow plugin code to refer to them.
	static const FName SourcesTabId = "SourcesTabId";
	static const FName SourceDetailsTabId = "SourceDetailsTabId";
	static const FName SubjectsTabId = "SubjectsTabId";
	static const FName SubjectDetailsTabId = "SubjectDetailsTabId";
	static const FName PlaybackTabId = "PlaybackBottomTabId";
	static const FName RecordingListTabId = "RecordingListTabId";
	static const FName ClientsTabId = "ClientsTabId";
	static const FName ClientDetailsTabId = "ClientDetailsTabId";
}

class FLiveLinkHubApplicationMode;

/** Base class for the live link hub application, meant to expose a limited interface to modes that extend FLiveLinkHubApplicationMode. */
class FLiveLinkHubApplicationBase : public FWorkflowCentricApplication
{
public:
	virtual ~FLiveLinkHubApplicationBase() = default;

	/** Add a LiveLinkHub application mode. */
	virtual void AddLiveLinkHubApplicationMode(FName ModeName, TSharedRef<FLiveLinkHubApplicationMode> Mode) = 0;

	/** Push tab factories for the given mode. */
	virtual void PushTabFactories(const class FWorkflowAllowedTabSet& FactorySetToPush, TSharedPtr<FLiveLinkHubApplicationMode> ApplicationMode) = 0;
};


