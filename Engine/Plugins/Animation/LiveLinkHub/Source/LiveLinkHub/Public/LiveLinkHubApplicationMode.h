// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"

#include "LiveLinkHubApplicationBase.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API LIVELINKHUB_API

struct FSlateIcon;
class FToolBarBuilder;
class FWorkflowCentricApplication;
class FLiveLinkHubApplicationBase;
class SWidget;

/** Factory used by external plugins to register custom LiveLinkHub layouts. */
class ILiveLinkHubApplicationModeFactory : public IModularFeature
{
public:
	/** Name of the modular feature. */
	static UE_API const FName ModularFeatureName;

	/** Instantiate an application mode so LiveLinkHub can register it and display it in its Layout Selector. */
	virtual TSharedRef<class FLiveLinkHubApplicationMode> CreateLiveLinkHubAppMode(TSharedPtr<FLiveLinkHubApplicationBase> InApp) = 0;
};

class FLiveLinkHubApplicationMode : public FApplicationMode
{
public:
	/** Name of the File menu extension point. Can be used to insert menus before/after in the main menu bar. */
	static UE_API const FName FileMenuExtensionPoint;

	/** Get the icon for this mode. */
	virtual FSlateIcon GetModeIcon() const { return FSlateIcon(); }

	/** Get the display name for this mode */
	const FText& GetDisplayName() const { return DisplayName; }

	/** Method used to gather the toolbar widgets that should be visible when this mode is active. */
	UE_API TArray<TSharedRef<SWidget>> GetToolbarWidgets();

	/** Get the name of this layout. */
	FName GetLayoutName()
	{
		return TabLayout->GetLayoutName();
	}

	/** Get the filename of this mode's layout ini. */
	FString GetLayoutIni() const
	{
		return LayoutIni;
	}

	/** Get the tab manager layout ptr. */
	TSharedPtr<FTabManager::FLayout> GetTabLayout() const
	{
		return TabLayout;
	}

	/** Returns whether this mode is a user layout. */
	virtual bool IsUserLayout() const
	{
		return false;
	}

	/** Override to provide a custom Window menu section for this mode's tabs instead of the shared
	 *  "Live Link Hub" section. Return nullptr to use the shared section (default behaviour). */
	virtual TSharedPtr<FWorkspaceItem> GetCustomWindowMenuCategory() const { return nullptr; }

	/** Implement in a child class in order to have status bar extensions show up when the mode is active. */
	virtual TArray<TSharedRef<SWidget>> GetToolbarWidgets_Impl() { return {}; };


	/** Get widgets that should appear in the status bar. */
	virtual TArray<TSharedRef<SWidget>> GetStatusBarWidgets_Impl() { return {}; }

protected:
	UE_API FLiveLinkHubApplicationMode(FName InApplicationMode, FText InDisplayName, TSharedPtr<FLiveLinkHubApplicationBase> InApp);

	//~ Begin FApplicationMode interface
	UE_API virtual void PostActivateMode() override;
	UE_API virtual void PreDeactivateMode() override;
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override
	{
		if (!bRegisteredTabFactories)
		{
			if (TSharedPtr<FLiveLinkHubApplicationBase> App = WeakApp.Pin())
			{
				// Pushing the tab factories will register the tab spwaners for the current mode.
				App->PushTabFactories(TabFactories, StaticCastSharedRef<FLiveLinkHubApplicationMode>(AsShared()));
			}

			bRegisteredTabFactories = true;
		}
	}

protected:
	/** The application that hosts this app mode. */
	TWeakPtr<FLiveLinkHubApplicationBase> WeakApp;

	/** Set of spawnable tabs in the mode */
	FWorkflowAllowedTabSet TabFactories;

private:
	/** Display name for this mode. */
	FText DisplayName;

	/** Keeps track of whether or not the tab factories were registered, so we don't end up with duplicate tab spawners for this mode. */
	bool bRegisteredTabFactories = false;
};

#undef UE_API
