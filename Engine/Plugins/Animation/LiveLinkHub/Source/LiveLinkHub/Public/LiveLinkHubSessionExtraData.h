// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "Include LiveLinkHub/LiveLinkHubSessionExtraData.h from the LiveLinkInterface module instead.")

#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"
