// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkHubEditor : ModuleRules
	{
		public LiveLinkHubEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Json",
				"Engine",
				"LauncherPlatform",
				"LiveLink",
				"LiveLinkEditor",
				"LiveLinkHub",
				"LiveLinkHubMessaging",
				"LiveLinkInterface",
				"Serialization",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd"
			});
		}
	}
}
