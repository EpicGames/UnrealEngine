// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubClientIdCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "ILiveLinkHubClientsModel.h"
#include "Input/Events.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubClientIdCustomization"


class SLiveLinkHubClientIdDropdown : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FLiveLinkHubClientId);

public:
	SLATE_BEGIN_ARGS(SLiveLinkHubClientIdDropdown)
		: _MinDesiredValueWidth( 40 )
		{
		}

		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		/** Controls the minimum width for the text box portion of the control. */
		SLATE_ATTRIBUTE(float, MinDesiredValueWidth)

		/** The callback for when the value changes. */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		PropertyHandle = InArgs._PropertyHandle;
		OnValueChanged = InArgs._OnValueChanged;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SComboButton)
					.ContentPadding(1)
					.OnGetMenuContent(this, &SLiveLinkHubClientIdDropdown::BuildMenu)
					.ButtonContent()
					[
						SNew(STextBlock)
						.MinDesiredWidth(InArgs._MinDesiredValueWidth)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &SLiveLinkHubClientIdDropdown::GetValueText)
					]
				]
			]
		];
	}

private:
	TSharedRef<SWidget> BuildMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		if (ILiveLinkHubClientsModel* ClientsModel = ILiveLinkHubClientsModel::Get())
		{
			TArray<FLiveLinkHubClientId> SessionClients = ClientsModel->GetSessionClients();
			for (FLiveLinkHubClientId ClientId : SessionClients)
			{
				FUIAction MenuAction(FExecuteAction::CreateSP(this, &SLiveLinkHubClientIdDropdown::SetValue, ClientId));
				MenuBuilder.AddMenuEntry(ClientsModel->GetClientDisplayName(ClientId), FText::GetEmpty(), FSlateIcon(), MenuAction);
			}
		}

		return MenuBuilder.MakeWidget();
	}

	void SetValue(FLiveLinkHubClientId InNewClientId)
	{
		if (PropertyHandle && PropertyHandle->IsValidHandle())
		{
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty());

			TArray<void*> RawData;
			PropertyHandle->AccessRawData(RawData);
			if (RawData.Num() > 0 && RawData[0])
			{
				FLiveLinkHubClientId* PreviousValue = reinterpret_cast<FLiveLinkHubClientId*>(RawData[0]);

				FString TextValue;
				StructProperty->Struct->ExportText(TextValue, &InNewClientId, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
				ensure(PropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
			}
		}
	}

	FText GetValueText() const
	{
		if (PropertyHandle && PropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			PropertyHandle->AccessRawData(RawData);
			if (RawData.Num() == 1 && RawData[0])
			{
				return GetClientText(*(const FLiveLinkHubClientId*)RawData[0]);
			}
			else
			{
				return LOCTEXT("ValueText_MultipleValues", "Multiple Values");
			}
		}

		return LOCTEXT("ValueText_HandleInvalid", "(Error: Invalid property handle)");
	}

	static FText GetClientText(FLiveLinkHubClientId InClientId)
	{
		if (InClientId.IsValid())
		{
			if (ILiveLinkHubClientsModel* ClientsModel = ILiveLinkHubClientsModel::Get())
			{
				return ClientsModel->GetClientDisplayName(InClientId);
			}
			else
			{
				return LOCTEXT("ClientText_ClientsModelInvalid", "(Error: No clients model)");
			}
		}
		else
		{
			return LOCTEXT("ClientText_IdInvalid", "(None)");
		}
	}

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	FOnValueChanged OnValueChanged;
};


//////////////////////////////////////////////////////////////////////////


TSharedRef<IPropertyTypeCustomization> FLiveLinkHubClientIdCustomization::MakeInstance()
{
	return MakeShared<FLiveLinkHubClientIdCustomization>();
}


void FLiveLinkHubClientIdCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InStructPropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils
)
{
	StructPropertyHandle = InStructPropertyHandle;
	FString CurrentValue;
	StructPropertyHandle->GetValueAsFormattedString(CurrentValue);

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SBox)
		.Padding(FMargin(0,2,0,1))
		[
			SNew(SLiveLinkHubClientIdDropdown)
			.PropertyHandle(StructPropertyHandle)
		]
	];
}


void FLiveLinkHubClientIdCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InStructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils
)
{
}


#undef LOCTEXT_NAMESPACE
