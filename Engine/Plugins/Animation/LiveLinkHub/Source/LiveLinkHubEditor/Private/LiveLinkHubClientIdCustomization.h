// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"


class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;


class FLiveLinkHubClientIdCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual bool ShouldInlineKey() const override { return true; }

private:	
	/** Cached property handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
