// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubEditorModule.h"

#include "DesktopPlatformModule.h"
#include "HAL/FileManager.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "LiveLinkHubClientIdCustomization.h"
#include "LiveLinkHubLauncherUtils.h"
#include "LiveLinkHubEditorSettings.h"
#include "Misc/App.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/MessageDialog.h"
#include "PropertyEditorModule.h"
#include "Runtime/Launch/Resources/Version.h"
#include "SLiveLinkHubEditorStatusBar.h"
#include "ToolMenus.h"


static TAutoConsoleVariable<int32> CVarLiveLinkHubEnableStatusBar(
	TEXT("LiveLinkHub.EnableStatusBar"), 1,
	TEXT("Whether to enable showing the livelink hub status bar in the editor. Must be set before launching the editor."),
	ECVF_RenderThreadSafe);

DECLARE_LOG_CATEGORY_CLASS(LogLiveLinkHubEditor, Log, Log)

#define LOCTEXT_NAMESPACE "LiveLinkHubEditor"

void FLiveLinkHubEditorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLiveLinkHubEditorModule::OnPostEngineInit);
}

void FLiveLinkHubEditorModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	if (!IsRunningCommandlet() && CVarLiveLinkHubEnableStatusBar.GetValueOnAnyThread()) 
	{	
		UnregisterLiveLinkHubStatusBar();
	}

	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout("LiveLinkHubClientId");
	}
}

void FLiveLinkHubEditorModule::OnPostEngineInit()
{
	if (!IsRunningCommandlet() && CVarLiveLinkHubEnableStatusBar.GetValueOnAnyThread())
	{
		if (GEditor && FApp::CanEverRender())
		{
			RegisterLiveLinkHubStatusBar();
		
			FToolMenuOwnerScoped OwnerScoped(this);
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
			FToolMenuSection& Section = Menu->AddSection("VirtualProductionSection", LOCTEXT("VirtualProductionSection", "Virtual Production"));

			Section.AddMenuEntry("LiveLinkHub",
				LOCTEXT("LiveLinkHubLabel", "Live Link Hub"),
				LOCTEXT("LiveLinkHubTooltip", "Launch the Live Link Hub app."),
				FSlateIcon("LiveLinkStyle", "LiveLinkClient.Common.Icon.Small"),
				FUIAction(FExecuteAction::CreateStatic(&UE::LiveLinkHubLauncherUtils::OpenLiveLinkHub)));
		}
	}

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("LiveLinkHubClientId", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkHubClientIdCustomization::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();
}

void FLiveLinkHubEditorModule::RegisterLiveLinkHubStatusBar()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("LevelEditor.StatusBar.ToolBar"));

	FToolMenuSection& LiveLinkHubSection = Menu->AddSection(TEXT("LiveLinkHub"), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	LiveLinkHubSection.AddEntry(
		FToolMenuEntry::InitWidget(TEXT("LiveLinkHubStatusBar"), CreateLiveLinkHubWidget(), FText::GetEmpty(), true, false)
	);
}

void FLiveLinkHubEditorModule::UnregisterLiveLinkHubStatusBar()
{
	UToolMenus::UnregisterOwner(this);
}

TSharedRef<SWidget> FLiveLinkHubEditorModule::CreateLiveLinkHubWidget()
{
	return SNew(SLiveLinkHubEditorStatusBar);
}

IMPLEMENT_MODULE(FLiveLinkHubEditorModule, LiveLinkHubEditor);

#undef LOCTEXT_NAMESPACE /*LiveLinkHubEditor*/ 
