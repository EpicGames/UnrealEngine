// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleInterface.h"

/**
 * Editor module responsible for creating the hub's status bar.
 */
class FLiveLinkHubEditorModule : public IModuleInterface
{

public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule();

	virtual void ShutdownModule();
	//~ End IModuleInterface interface

private:
	/** Callback used to register the livelink hub status bar. */
	void OnPostEngineInit();

	/** Extend the editor's bottom status bar to add the livelink hub widget. */
	void RegisterLiveLinkHubStatusBar();

	/** Unregister our tool menu customization. */
	void UnregisterLiveLinkHubStatusBar();

	/** Instantiate the livelink hub status bar widget. */
    TSharedRef<class SWidget> CreateLiveLinkHubWidget();

	/** Path to livelink hub. */
	FString LiveLinkHubExecutablePath;
};
