// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "LiveLinkHubEditorSettings.generated.h"

/**
 * Settings for LiveLinkHub in editor.
 */
UCLASS(MinimalAPI, config = Editor, defaultconfig)
class ULiveLinkHubEditorSettings : public UObject
{
public:
	GENERATED_BODY()

	/** Whether to find the livelinkhub executable by looking up a registry key. */
	UPROPERTY(config)
	bool bDetectLiveLinkHubExecutable = false;

	/** Relative URI to the LiveLinkHub page in the epic games launcher. */
	UPROPERTY(config)
	FString LiveLinkHubStorePage;

	/** App name of LiveLinkHub, used to automatically detect if it's installed. */
	UPROPERTY(config)
	FString LiveLinkHubAppName;

	/** The version of LiveLinkHub that should be used. If empty, any version will be allowed. */
	UPROPERTY(config)
	FString LiveLinkHubTargetVersion;
};
