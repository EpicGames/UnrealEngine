// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubLauncherUtils.h"
#include "LiveLinkHubEditorSettings.h"

#include "Containers/StringConv.h"
#include "HAL/FileManager.h"
#include "LauncherPlatformModule.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/MessageDialog.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


DECLARE_LOG_CATEGORY_CLASS(LogLiveLinkHubLauncher, Log, Log)

#define LOCTEXT_NAMESPACE "LiveLinkHubLauncher"


bool UE::LiveLinkHubLauncherUtils::FindLiveLinkHubInstallation(FInstalledApp& OutLiveLinkHubInfo)
{
	FString InstalledListFile = FString(FPlatformProcess::ApplicationSettingsDir()) / TEXT("UnrealEngineLauncher/LauncherInstalled.dat");

	FString InstalledText;
	if (FFileHelper::LoadFileToString(InstalledText, *InstalledListFile))
	{
		// Deserialize the object
		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InstalledText);
		if (FJsonSerializer::Deserialize(Reader, RootObject) && RootObject.IsValid())
		{
			// Parse the list of installations
			TArray<TSharedPtr<FJsonValue>> InstallationList = RootObject->GetArrayField(TEXT("InstallationList"));
			for (int32 Idx = 0; Idx < InstallationList.Num(); Idx++)
			{
				FInstalledApp InstalledApp;

				TSharedPtr<FJsonObject> InstallationItem = InstallationList[Idx]->AsObject();
				InstalledApp.AppName = InstallationItem->GetStringField(TEXT("AppName"));

				if (InstalledApp.AppName == GetDefault<ULiveLinkHubEditorSettings>()->LiveLinkHubAppName)
				{
					InstalledApp.InstallLocation = InstallationItem->GetStringField(TEXT("InstallLocation"));

					if (!InstalledApp.InstallLocation.Len())
					{
						// Shouldn't happen in theory, but just to be safe. 
						// Doing a continue here instead of returning in case there were somehow multiple LLH installations.
						continue;
					}

					const FString& TargetVersion = GetDefault<ULiveLinkHubEditorSettings>()->LiveLinkHubTargetVersion;
					InstalledApp.AppVersion = InstallationItem->GetStringField(TEXT("AppVersion"));

					if (!TargetVersion.IsEmpty() && InstalledApp.AppVersion != TargetVersion)
					{
						// If we target a specific version and it doesn't match the installed app, ignore it.
						continue;
					}

					InstalledApp.NamespaceId = InstallationItem->GetStringField(TEXT("NamespaceId"));
					InstalledApp.ItemId = InstallationItem->GetStringField(TEXT("ItemId"));
					InstalledApp.ArtifactId = InstallationItem->GetStringField(TEXT("ArtifactId"));


					OutLiveLinkHubInfo = MoveTemp(InstalledApp);
					return true;
				}
			}
		}
	}

	return false;
}

void UE::LiveLinkHubLauncherUtils::OpenLiveLinkHub()
{
	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("LaunchingLiveLinkHub", "Launching Live Link Hub...");
	NotificationConfig.LogCategory = &LogLiveLinkHubLauncher;

	FAsyncTaskNotification Notification(NotificationConfig);
	const FText LaunchLiveLinkHubErrorTitle = LOCTEXT("LaunchLiveLinkHubErrorTitle", "Failed to Launch LiveLinkhub.");

	// Try getting the livelinkhub app location by reading a registry key.
	if (GetDefault<ULiveLinkHubEditorSettings>()->bDetectLiveLinkHubExecutable)
	{
		ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

		UE::LiveLinkHubLauncherUtils::FInstalledApp LiveLinkHubApp;
		if (UE::LiveLinkHubLauncherUtils::FindLiveLinkHubInstallation(LiveLinkHubApp))
		{
			// Found a LiveLinkHub installation from the launcher, so launch it that way.

			const FString LaunchLink = TEXT("apps") / LiveLinkHubApp.NamespaceId + TEXT("%3A") + LiveLinkHubApp.ItemId + TEXT("%3A") + LiveLinkHubApp.AppName + TEXT("?action=launch&silent=true");
			FOpenLauncherOptions OpenOptions(LaunchLink);
			if (!LauncherPlatform->OpenLauncher(OpenOptions))
			{
				Notification.SetComplete(
					LaunchLiveLinkHubErrorTitle,
					LOCTEXT("LaunchLiveLinkHubError_CouldNotOpenLauncher", "Could not launch Live Link Hub through the Epic Games Store."),
					false
				);

			}
			else
			{
				Notification.SetComplete(
					LOCTEXT("LiveLinkHubLaunchSuccessTitle", "Launched Live Link Hub."),
					LOCTEXT("LaunchLiveLinkHubError_LaunchSuccess", "Launching Liv Link Hub through the Epic Games Store."),
					true
				);
			}
		}
		else
		{
			const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("InstallThroughEGS", "Live Link Hub is not currently installed, do you want to install it through the Epic Games Store?"));

			if (Choice == EAppReturnType::Yes)
			{
				// Could not find LiveLinkHub from the launcher. Prompt the user to open the EGS and install it.
				if (!GetDefault<ULiveLinkHubEditorSettings>()->LiveLinkHubStorePage.IsEmpty())
				{
					FOpenLauncherOptions OpenOptions(GetDefault<ULiveLinkHubEditorSettings>()->LiveLinkHubStorePage);
					if (!LauncherPlatform->OpenLauncher(OpenOptions))
					{
						Notification.SetComplete(
							LaunchLiveLinkHubErrorTitle,
							LOCTEXT("LaunchLiveLinkHubError_CouldNotFindHubStorePage", "Could not find the Live Link Hub page on the Epic Games Store."),
							false
						);
					}
					else
					{
						Notification.SetComplete(
							LaunchLiveLinkHubErrorTitle,
							LOCTEXT("LaunchLiveLinkHub_LaunchFromStore", "Opening Epic Games Store to the Live Link Hub page."),
							true
						);
					}

				}
				else
				{
					Notification.SetComplete(
						LaunchLiveLinkHubErrorTitle,
						LOCTEXT("LaunchLiveLinkHubError_EmptyConfig", "Could not find the Live Link Hub page on the Epic Games Store, missing configuration for the store page."),
						false
					);

				}
			}
			else
			{
				Notification.SetComplete(
					LaunchLiveLinkHubErrorTitle,
					LOCTEXT("LaunchLiveLinkHub_DidNotLaunchFromStore", "Live Link Hub could not be launched since it wasn't installed."),
					false
				);
			}

		}

		return;
	}

	// Find livelink hub executable location for our build configuration
	FString LiveLinkHubPath = FPlatformProcess::GenerateApplicationPath(TEXT("LiveLinkHub"), FApp::GetBuildConfiguration());

	// Validate it exists and fall back to development if it doesn't.
	if (!IFileManager::Get().FileExists(*LiveLinkHubPath))
	{
		LiveLinkHubPath = FPlatformProcess::GenerateApplicationPath(TEXT("LiveLinkHub"), EBuildConfiguration::Development);

		// If it still doesn't exist, fall back to the shipping executable.
		if (!IFileManager::Get().FileExists(*LiveLinkHubPath))
		{
			LiveLinkHubPath = FPlatformProcess::GenerateApplicationPath(TEXT("LiveLinkHub"), EBuildConfiguration::Shipping);
		}
	}

	if (!IFileManager::Get().FileExists(*LiveLinkHubPath))
	{
		Notification.SetComplete(
			LaunchLiveLinkHubErrorTitle,
			LOCTEXT("LaunchLiveLinkHubError_ExecutableMissing", "Could not find the executable. Have you compiled the Live Link Hub app?"),
			false
		);

		return;
	}

	// Validate we do not have it running locally
	const FString AppName = FPaths::GetCleanFilename(LiveLinkHubPath);
	if (FPlatformProcess::IsApplicationRunning(*AppName))
	{
		Notification.SetComplete(
			LaunchLiveLinkHubErrorTitle,
			LOCTEXT("LaunchLiveLinkHubError_AlreadyRunning", "A Live Link Hub instance is already running."),
			false
		);
		return;
	}

	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;

	const FProcHandle ProcHandle = FPlatformProcess::CreateProc(*LiveLinkHubPath, TEXT(""), bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, nullptr, nullptr);
	if (ProcHandle.IsValid())
	{
		Notification.SetComplete(
			LOCTEXT("LaunchedLiveLinkHub", "Launched Live Link Hub"), FText(), true);

		return;
	}
	else // Very unlikely in practice, but possible in theory.
	{
		Notification.SetComplete(
			LaunchLiveLinkHubErrorTitle,
			LOCTEXT("LaunchLiveLinkHubError_InvalidHandle", "Failed to create the Live Link Hub process."),
			false);
	}
}

#undef LOCTEXT_NAMESPACE /*LiveLinkHubLauncher*/ 
