// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkHubEditorStatusBar.h"

#include "Editor.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkHubMessagingModule.h"
#include "ILiveLinkModule.h"
#include "ILiveLinkSource.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkEditorSettings.h"
#include "LiveLinkMessageBusSource.h"
#include "LiveLinkPreset.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/SlateStyle.h"
#include "TimerManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubStatusBar"

void SLiveLinkHubEditorStatusBar::Construct(const FArguments& InArgs)
{
	ILiveLinkHubMessagingModule& HubMessagingModule = FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");
	HubMessagingModule.OnConnectionEstablished().AddSP(this, &SLiveLinkHubEditorStatusBar::OnHubConnectionEstablished);

	LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	LiveLinkClient->OnLiveLinkSourceRemoved().AddSP(this, &SLiveLinkHubEditorStatusBar::OnSourceRemoved);
	LiveLinkClient->OnLiveLinkSubjectRemoved().AddSP(this, &SLiveLinkHubEditorStatusBar::OnSubjectRemoved);
	LiveLinkClient->OnLiveLinkSourcesChanged().AddRaw(this, &SLiveLinkHubEditorStatusBar::RefreshSubjects);
	LiveLinkClient->OnLiveLinkSubjectsChanged().AddRaw(this, &SLiveLinkHubEditorStatusBar::RefreshSubjects);

	IModularFeatures::Get().OnModularFeatureUnregistered().AddSPLambda(this,
		[this](const FName& InFeatureName, IModularFeature* InFeature)
		{
			if (InFeature == this->LiveLinkClient)
			{
				this->LiveLinkClient = nullptr;
			}
		}
	);

	constexpr bool bLoop = true;
	if (GEditor && GEditor->IsTimerManagerValid())
	{
		GEditor->GetTimerManager()->SetTimer(TimerHandle, FTimerDelegate::CreateSP(this, &SLiveLinkHubEditorStatusBar::CheckHubConnection), CheckConnectionIntervalSeconds, bLoop);
	}

	auto NullLambda = [](TSharedPtr<struct FLiveLinkSubjectUIEntry>, ESelectInfo::Type) {};
	TWeakPtr<SLiveLinkHubEditorStatusBar> WeakStatusBar = StaticCastSharedRef<SLiveLinkHubEditorStatusBar>(AsShared());
	TAttribute<bool> ReadOnlyLambda = TAttribute<bool>::CreateLambda([WeakStatusBar]()
	{
		if (TSharedPtr<SLiveLinkHubEditorStatusBar> StatusBar = WeakStatusBar.Pin())
		{
			return StatusBar->ConnectionState == EHubConnectionState::Timeout || GetDefault<ULiveLinkEditorSettings>()->bReadOnly;
		}
		return true;
	});


	SubjectsView = MakeShared<FLiveLinkSubjectsView>(FLiveLinkSubjectsView::FOnSubjectSelectionChanged::CreateLambda(MoveTemp(NullLambda)), nullptr, ReadOnlyLambda);

	RefreshSubjects();

	ChildSlot
	[
		SNew(SComboButton)
    		.ContentPadding(FMargin(6.0f, 0.0f))
    		.MenuPlacement(MenuPlacement_AboveAnchor)
    		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.HasDownArrow(false)
			.Visibility(this, &SLiveLinkHubEditorStatusBar::GetVisibility)
    		.ButtonContent()
    		[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 3, 0)
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SLiveLinkHubEditorStatusBar::GetIconColor)
					.Image(FSlateIcon("LiveLinkStyle","LiveLinkClient.Common.Icon.Small").GetIcon())
				]
				+ SHorizontalBox::Slot()
                .AutoWidth()
                .VAlign(VAlign_Center)
                .Padding(2, 0, 10, 0)
                [
                	SNew(STextBlock)
                	.Text(this, &SLiveLinkHubEditorStatusBar::GetStatusText)
                	.ToolTipText(this, &SLiveLinkHubEditorStatusBar::GetToolTipText)
                ]
    		]
			.MenuContent()
			[
				SNew(SBox)
				.MinDesiredHeight(200.0f)
				.MinDesiredWidth(300.0f)
				[
					SubjectsView->SubjectsTreeView.ToSharedRef()
				]
			]
	];
}

SLiveLinkHubEditorStatusBar::~SLiveLinkHubEditorStatusBar()
{
	if (LiveLinkClient)
	{
		LiveLinkClient->OnLiveLinkSubjectsChanged().RemoveAll(this);
		LiveLinkClient->OnLiveLinkSourcesChanged().RemoveAll(this);
		LiveLinkClient->OnLiveLinkSubjectRemoved().RemoveAll(this);
		LiveLinkClient->OnLiveLinkSourceRemoved().RemoveAll(this);
	}

	if (ILiveLinkHubMessagingModule* HubMessagingModule = FModuleManager::Get().GetModulePtr<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging"))
	{
		HubMessagingModule->OnConnectionEstablished().RemoveAll(this);
	}

	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);

	if (GEditor && GEditor->IsTimerManagerValid())
	{
		GEditor->GetTimerManager()->ClearTimer(TimerHandle);
	}
}

FText SLiveLinkHubEditorStatusBar::GetStatusText() const
{
	const bool bConnectionActive = ConnectionState == EHubConnectionState::Valid;
	return bConnectionActive ? LOCTEXT("LiveLinkHubConnectionActive", "LiveLinkHub Connected") : LOCTEXT("LiveLinkHubConnectionError", "Live Link Hub Error");
}

FText SLiveLinkHubEditorStatusBar::GetToolTipText() const
{
	const bool bConnectionActive = ConnectionState == EHubConnectionState::Valid;
	return bConnectionActive ? LOCTEXT("LiveLinkHubConnectionActiveTooltip", "LiveLinkHub is connected to this instance of Unreal Editor.") : LOCTEXT("LiveLinkHubConnectionErrorTooltip", "The Live Link Hub connection was terminated.");
}

FSlateColor SLiveLinkHubEditorStatusBar::GetIconColor() const
{
	const bool bConnectionActive = ConnectionState == EHubConnectionState::Valid;
	return bConnectionActive ? FSlateColor(FLinearColor::Green) : FSlateColor(FLinearColor::Red);
}

void SLiveLinkHubEditorStatusBar::OnSourceRemoved(FGuid SourceId)
{
	SourceIds.Remove(SourceId);

	RefreshSubjects();
}

void SLiveLinkHubEditorStatusBar::OnSubjectRemoved(FLiveLinkSubjectKey)
{
	RefreshSubjects();
}

void SLiveLinkHubEditorStatusBar::RefreshSubjects()
{
	if (SubjectsView)
	{
		SubjectsView->RefreshSubjects();
	}
}

void SLiveLinkHubEditorStatusBar::OnHubConnectionEstablished(FGuid SourceId)
{
	bHasConnectedOnce = true;

	SourceIds.Add(SourceId);
	ConnectionState = EHubConnectionState::Valid;
}

void SLiveLinkHubEditorStatusBar::CheckHubConnection()
{
	if (SourceIds.IsEmpty())
	{
		ConnectionState = Invalid;
		return;
	}

	for (FGuid SourceId : SourceIds)
	{
		if (LiveLinkClient && LiveLinkClient->IsSourceStillValid(SourceId))
		{
			ConnectionState = EHubConnectionState::Valid;
			return;
		}
	}

	ConnectionState = EHubConnectionState::Timeout;
}

EVisibility SLiveLinkHubEditorStatusBar::GetVisibility() const
{
	if (ConnectionState != EHubConnectionState::Invalid || bHasConnectedOnce)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE /*LiveLinkHubStatusBar*/
