// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Internationalization/Text.h"
#include "LiveLinkTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ILiveLinkSource;

enum EHubConnectionState
{
	Invalid,
	Valid,
	Timeout
};

/** Widget responsible for showing the current connection status with the hub. Will only appear if a connection has been established with a hub instance. */
class SLiveLinkHubEditorStatusBar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkHubEditorStatusBar) {}
	SLATE_END_ARGS()

	~SLiveLinkHubEditorStatusBar() override;

	void Construct(const FArguments& InArgs);

private:
	/** Get the status text for the widget. */
	FText GetStatusText() const;
	/** Get the tooltip text for the widget. */
	FText GetToolTipText() const;
	/** Get the icon color for the widget according to the current status. */
	FSlateColor GetIconColor() const;

	/** Handler used to update the current connection state. */
	void OnSourceRemoved(FGuid SourceId);

	/** Handler used to update the subject list displayed by the status bar. */
	void OnSubjectRemoved(FLiveLinkSubjectKey SubjectKey);

	/** Handler used to update the connection state and source id when a connection with a hub instance is established. */
	void OnHubConnectionEstablished(FGuid SourceId);

	/** Check whether the hub connection is still active. */
	void CheckHubConnection();

	/** Returns the visibility of the widget (Depends on whether we've established connection). */
	EVisibility GetVisibility() const;

	/** Method called when the subjects or sources have been modified, used to refresh the subject list UI. */
	void RefreshSubjects();

private:
	/** LiveLink client used to retrieve the status of the hub connection. */
	class ILiveLinkClient* LiveLinkClient = nullptr;

	/** Whether the editor was able to connect to a hub, will always show the status bar icon after this. */
	bool bHasConnectedOnce = false;

	/** Ids of the livelinkhub sources connected to this editor. */
	TSet<FGuid> SourceIds;

	/** Current connection state to the hub. */
	EHubConnectionState ConnectionState = Invalid;

	/** Handle to the timer responsible for triggering CheckHubConnection. */
	FTimerHandle TimerHandle;

	/** Interval of the timer to check for connection validity. */
	static constexpr float CheckConnectionIntervalSeconds = 1.0f;

	/** Object responsible for creating and handling the subject list widget. */
	TSharedPtr<class FLiveLinkSubjectsView> SubjectsView;
};
