// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::LiveLinkHubLauncherUtils
{
	struct FInstalledApp
	{
		/** Location of the installed app. */
		FString InstallLocation;

		/** Namespace of the app. */
		FString NamespaceId;

		/** Id of the app. */
		FString ItemId;

		/** Unique ID for the app on the EGS. */
		FString ArtifactId;

		/** Version of the app. For LiveLinkHub this will correspond to a CL number.  */
		FString AppVersion;

		/** The apps' internal name. Usually matches the ArtifactId except if the app was using a legacy publishing workflow.  */
		FString AppName;
	};

	/** Gather all the installed apps from the Epic Games Launcher. */
	bool LIVELINKHUBEDITOR_API FindLiveLinkHubInstallation(FInstalledApp& OutLiveLinkHubInfo);

	/** Launch the livelinkhub executable. */
	void LIVELINKHUBEDITOR_API OpenLiveLinkHub();
}
