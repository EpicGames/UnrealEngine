// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkHubMessaging : ModuleRules
	{
		public LiveLinkHubMessaging(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"TimeManagement",
				"LiveLinkMessageBusFramework"
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"LiveLink",
				"LiveLinkInterface",
				"Messaging",
				"SlateCore"
			});

			if (Target.bCompileAgainstEditor)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Settings",
					"Slate",
					"UnrealEd"
				});
			}
		}
	}
}
