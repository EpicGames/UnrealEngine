// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "Features/IModularFeatures.h"
#include "HAL/PlatformProcess.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkModule.h"
#include "ILiveLinkSource.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkHubMessageBusSource.h"
#include "LiveLinkHubMessagingModule.h"
#include "LiveLinkHubMessagingSettings.h"
#include "LiveLinkMessageBusDiscoveryManager.h"
#include "LiveLinkMessageBusFinder.h"
#include "LiveLinkSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "TimerManager.h"
#include "UObject/UObjectGlobals.h"


#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHubConnectionManager, Log, All);

namespace LiveLinkHubConnectionManager
{
	template <typename T>
	concept CHasAnnotations = requires(const T& t) { t.GetAnnotations(); };

	static void SendAnalyticsConnectionEstablished()
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.LiveLinkHub.ConnectionEstablished"), {});
	}

	static ELiveLinkTopologyMode GetPollResultTopologyMode(const FProviderPollResultPtr& PollResult)
	{
		// Default to Hub since all LiveLinkHub instances were hubs before spokes were introduced.
		ELiveLinkTopologyMode PollResultMode = ELiveLinkTopologyMode::Hub;

		if (PollResult)
		{
			if (const FString* TopologyModeAnnotation = PollResult->Annotations.Find(FLiveLinkMessageAnnotation::TopologyModeAnnotation))
			{
				int64 TopologyModeValue = StaticEnum<ELiveLinkTopologyMode>()->GetValueByName(**TopologyModeAnnotation);
				if (TopologyModeValue != INDEX_NONE)
				{
					PollResultMode = (ELiveLinkTopologyMode)TopologyModeValue;
				}
			}
			else if (PollResult->Annotations.FindRef(FLiveLinkHubMessageAnnotation::ProviderTypeAnnotation) != UE::LiveLinkHub::Private::LiveLinkHubProviderType)
			{
				// Non-Hub livelink providers are usually external if they don't have annotations.
				PollResultMode = ELiveLinkTopologyMode::External;
			}
		}

		return PollResultMode;
	}

	template<CHasAnnotations T>
	static bool CanConnectTo(const FString& MachineName, const T& ObjectWithAnnotations, const FLiveLinkHubInstanceId& InstanceId)
	{
		const TMap<FName, FString>& Annotations = ObjectWithAnnotations.GetAnnotations();
		ELiveLinkHubAutoConnectMode AutoConnectMode = ELiveLinkHubAutoConnectMode::All;
		if (const FString* AutoConnectModeAnnotation = Annotations.Find(FLiveLinkHubMessageAnnotation::AutoConnectModeAnnotation))
		{
			int64 AutoConnectModeValue = StaticEnum<ELiveLinkHubAutoConnectMode>()->GetValueByName(**AutoConnectModeAnnotation);
			if (AutoConnectModeValue != INDEX_NONE)
			{
				AutoConnectMode = (ELiveLinkHubAutoConnectMode)AutoConnectModeValue;
			}
		}

		// Prevent connecting to itself.
		bool bSameInstance = false;
		if (const FString* InstanceIdAnnotation = Annotations.Find(FLiveLinkHubMessageAnnotation::IdAnnotation))
		{
			bSameInstance = *InstanceIdAnnotation == InstanceId.ToString();
		}

		const bool bSameHost = MachineName == FPlatformProcess::ComputerName();

		// An explicit connect request (the user picked this client in the LLH "Add Client" menu) bypasses the auto-connect filter.
		const bool bExplicitConnect = Annotations.Contains(FLiveLinkHubMessageAnnotation::ExplicitConnectAnnotation);

		const bool bAutoConnectMatchResult = !bSameInstance &&
			(bExplicitConnect
				|| AutoConnectMode == ELiveLinkHubAutoConnectMode::All
				|| (AutoConnectMode == ELiveLinkHubAutoConnectMode::LocalOnly && bSameHost));
		if (!bAutoConnectMatchResult)
		{
			const FText AutoConnectModeName = UEnum::GetDisplayValueAsText(AutoConnectMode);
			UE_LOGF(LogLiveLinkHubConnectionManager, Verbose, "Refusing connection from incoming instance since it was in mode: %ls", *AutoConnectModeName.ToString());
		}

		return bAutoConnectMatchResult;
	}

	/** Returns whether this connection manager should accept connection requests from this poll result. */
	static bool ShouldAcceptConnectionFrom(ELiveLinkTopologyMode InHostMode, const FProviderPollResultPtr& InPollResult, const FLiveLinkHubInstanceId& InstanceId)
	{
		if (!InPollResult)
		{
			return false;
		}

		// Topology Mode
		ELiveLinkTopologyMode IncomingMode = LiveLinkHubConnectionManager::GetPollResultTopologyMode(InPollResult);

		const bool bCompatibleModeResult = GetDefault<ULiveLinkHubMessagingSettings>()->CanReceiveFrom(InHostMode, IncomingMode);

		const FText ModeName = UEnum::GetDisplayValueAsText(InHostMode);
		const FText IncomingModeName = UEnum::GetDisplayValueAsText(IncomingMode);
		
		if (!bCompatibleModeResult)
		{
			UE_LOGF(LogLiveLinkHubConnectionManager, Verbose, "Refusing connection from incoming instance in %ls mode. This app is in %ls mode.", *IncomingModeName.ToString(), *ModeName.ToString());
		}

		return bCompatibleModeResult && LiveLinkHubConnectionManager::CanConnectTo(InPollResult->MachineName, *InPollResult, InstanceId);
	}
}

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD

DECLARE_DELEGATE_RetVal(ELiveLinkTopologyMode, FGetTopologyMode);
DECLARE_DELEGATE_RetVal(FLiveLinkHubInstanceId, FGetInstanceId);

/** This utitlity is meant to be run on an unreal engine instance to look for livelink hub connections and to automatically create the message bus source for it. */
class FLiveLinkHubConnectionManager
{
public:
	DECLARE_DELEGATE_RetVal(ELiveLinkTopologyMode, FOnGetTopologyMode);
	DECLARE_DELEGATE_RetVal(FLiveLinkHubInstanceId, FOnGetInstanceId);
	DECLARE_DELEGATE_OneParam(FOnDiscoveryRequest, const FMessageAddress& /** RemoteAddress */);


	FLiveLinkHubConnectionManager(ELiveLinkTopologyMode InMode, FOnGetTopologyMode OnGetTopologyMode, FOnGetInstanceId OnGetInstanceId, FOnDiscoveryRequest OnDiscoveryRequest)
		: GetTopologyModeDelegate(OnGetTopologyMode)
		, GetInstanceIdDelegate(OnGetInstanceId)
		, DiscoveryRequestDelegate(OnDiscoveryRequest)
	{
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FLiveLinkHubConnectionManager::PostLoadMap);
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLiveLinkHubConnectionManager::StartDiscovery);

		bEnableReconnectingToStaleSource = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bEnableReconnectingToStaleSource"), true, GEngineIni);
	}

	~FLiveLinkHubConnectionManager()
	{
		FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

		if (FTimerManager* TimerManager = GetTimerManager())
		{
			TimerManager->ClearTimer(ConnectionUpdateTimer);
		}

		if (ILiveLinkModule* LiveLinkModule = FModuleManager::GetModulePtr<ILiveLinkModule>("LiveLink"))
		{
			LiveLinkModule->GetMessageBusDiscoveryManager().RemoveDiscoveryMessageRequest();
		}
	}

private:
	/** Add a discovery request and start polling for results. */
	void StartDiscovery()
	{
		if (!ConnectionUpdateTimer.IsValid())
		{
			if (FTimerManager* TimerManager = GetTimerManager())
			{
				TimerManager->SetTimer(ConnectionUpdateTimer, FTimerDelegate::CreateRaw(this, &FLiveLinkHubConnectionManager::LookForLiveLinkHubConnection), GetDefault<ULiveLinkSettings>()->MessageBusPingRequestFrequency, true);
				ILiveLinkModule::Get().GetMessageBusDiscoveryManager().AddDiscoveryMessageRequest();
			}
		}
	}

	/** Get the timer manager either from the editor or the current world. */
	FTimerManager* GetTimerManager() const
	{
#if WITH_EDITOR
		if (GEditor && GEditor->IsTimerManagerValid())
		{
			return &GEditor->GetTimerManager().Get();
		}
		else
		{
			return GWorld ? &GWorld->GetTimerManager() : nullptr;
		}
#else
		return GWorld ? &GWorld->GetTimerManager() : nullptr;
#endif
	}

	/** Parse the poll results of the discovery manager and create a livelinkhub messagebus source if applicable. */
	void LookForLiveLinkHubConnection()
	{
		// Only look for a source if we don't have a valid connection.
		UE_LOGF(LogLiveLinkHubConnectionManager, Verbose, "Polling discovery results.");

		TArray<FProviderPollResultPtr> PollResults = ILiveLinkModule::Get().GetMessageBusDiscoveryManager().GetDiscoveryResults();

		for (const FProviderPollResultPtr& PollResult : PollResults)
		{
			const FString* ProviderType = PollResult->Annotations.Find(FLiveLinkHubMessageAnnotation::ProviderTypeAnnotation);

			if (ProviderType && *ProviderType == UE::LiveLinkHub::Private::LiveLinkHubProviderType)
			{
				if (PollResult->DiscoveryProtocolVersion > 1)
				{
					// DiscoveryProtocolV2: This comes from a provider built 5.7 or later, we don't want to add a source just yet, we'll just send a beacon back and let the provider initiate the connection.
					DiscoveryRequestDelegate.ExecuteIfBound(PollResult->Address);
				}
				else
				{
					const ELiveLinkTopologyMode HostMode = GetTopologyModeDelegate.Execute();
					const FLiveLinkHubInstanceId InstanceId = GetInstanceIdDelegate.Execute();
					if (LiveLinkHubConnectionManager::ShouldAcceptConnectionFrom(HostMode, PollResult, InstanceId))
					{
						AddLiveLinkSource(PollResult);
					}
				}
			}
		}
	}

	// Create a messagebus source 
	void AddLiveLinkSource(const FProviderPollResultPtr& PollResult)
	{
		UE_LOGF(LogLiveLinkHubConnectionManager, Verbose, "Discovered new source.");

		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

			for (const FGuid& SourceId : LiveLinkClient->GetSources())
			{
				if (LiveLinkClient->GetSourceType(SourceId).ToString() == PollResult->Name && LiveLinkClient->GetSourceMachineName(SourceId).ToString() == PollResult->MachineName)
				{
					// If we're reconnecting to an invalid source, make sure to delete the previous one first. 
					if (bEnableReconnectingToStaleSource && !LiveLinkClient->GetSourceStatus(SourceId).EqualToCaseIgnored(FLiveLinkMessageBusSource::ValidSourceStatus()))
					{
						// todo? We may want to eventually keep the source but "forwarding" the connection string to the source in order to keep the previous source settings.
						LiveLinkClient->RemoveSource(SourceId);
					}
					else
					{
						UE_LOGF(LogLiveLinkHubConnectionManager, Verbose, "Rejecting poll result since source %ls already exists.", *PollResult->Name);
						return;
					}
				}
			}

			ILiveLinkHubMessagingModule& HubMessagingModule = FModuleManager::GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");
			TSharedPtr<FLiveLinkHubMessageBusSource> LiveLinkSource = MakeShared<FLiveLinkHubMessageBusSource>(FText::FromString(PollResult->Name), FText::FromString(PollResult->MachineName), PollResult->Address, PollResult->MachineTimeOffset, PollResult->DiscoveryProtocolVersion);
			FGuid SourceId = LiveLinkClient->AddSource(LiveLinkSource);
			HubMessagingModule.OnConnectionEstablished().Broadcast(SourceId);
			LiveLinkHubConnectionManager::SendAnalyticsConnectionEstablished();
		}
		else
		{
			UE_LOGF(LogLiveLinkHubConnectionManager, Warning, "LiveLink modular feature was unavailable.");
		}
	}

	/** Handler called when a map changes, used to register the ConnectionUpdateTimer. */
	void PostLoadMap(UWorld*)
	{
		StartDiscovery();
	}
	
private:
	/** Handle to the timer used to check for livelink hub providers. */
	FTimerHandle ConnectionUpdateTimer;
	/** Get the mode for this connection manager. */
	FOnGetTopologyMode GetTopologyModeDelegate;
	/** Get the instance id (Only relevant if this is running inside of livelinkhub. */
	FOnGetInstanceId GetInstanceIdDelegate;
	/** Delegate called when we've received a Pong from a LiveLinkHub instance (that's built after 5.7 to support discovery). */
	FOnDiscoveryRequest DiscoveryRequestDelegate;
	/** Whether to allow reconnecting to stale LLH sources. */
	bool bEnableReconnectingToStaleSource = true;
	
};
#else
class FLiveLinkHubConnectionManager
{
};
#endif
