// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubMessageBusSource.h"

#include "Containers/Ticker.h"
#include "Engine/Level.h"
#include "Engine/SystemTimeTimecodeProvider.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "HAL/PlatformProcess.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "ILiveLinkModule.h"
#include "LiveLinkHubControlChannel.h"
#include "LiveLinkHubMessagingModule.h"
#include "LiveLinkHubMessages.h"
#include "LiveLinkMessages.h"
#include "LiveLinkTimecodeProvider.h"
#include "LiveLinkTypes.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"

#include <limits>

FLiveLinkHubMessageBusSource::FLiveLinkHubMessageBusSource(const FText& InSourceType, const FText& InSourceMachineName, const FMessageAddress& InConnectionAddress, double InMachineTimeOffset, int32 InDiscoveryProtocolVersion)
	: FLiveLinkMessageBusSource(InSourceType, InSourceMachineName, InConnectionAddress, InMachineTimeOffset)
	, CachedConnectionAddress(InConnectionAddress)
	, DiscoveryProtocolVersion(InDiscoveryProtocolVersion)
{
	if (DiscoveryProtocolVersion > 1)
	{
		// Since discovery request annotation was only present in the V2 of Discovery Protocol, give the global channel to the source.
		FLiveLinkHubMessagingModule& Module = static_cast<FLiveLinkHubMessagingModule&>(FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging"));
		ControlChannel = Module.GetControlChannel();
	}
	else
	{
		// For backwards compatibility, give its own control channel to the source.
		ControlChannel = MakeShared<FLiveLinkHubControlChannel>(FLiveLinkHubControlChannel::EChannelMode::Source);
	}
}

double FLiveLinkHubMessageBusSource::GetDeadSourceTimeout() const
{
	// Don't remove livelink hub sources that have hit the heartbeat timeout.
	return std::numeric_limits<double>::max();
}

bool FLiveLinkHubMessageBusSource::RequestSourceShutdown()
{
	ControlChannel->OnSourceShutdown(AsShared());

	return FLiveLinkMessageBusSource::RequestSourceShutdown();
}

void FLiveLinkHubMessageBusSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	FLiveLinkMessageBusSource::ReceiveClient(InClient, InSourceGuid);

	ControlChannel->Initialize(InSourceGuid);
}

void FLiveLinkHubMessageBusSource::InitializeMessageEndpoint(FMessageEndpointBuilder& EndpointBuilder)
{
	FLiveLinkMessageBusSource::InitializeMessageEndpoint(EndpointBuilder);

	if (ControlChannel->ChannelMode == FLiveLinkHubControlChannel::EChannelMode::Source)
	{
		ControlChannel->InitializeMessageEndpoint(EndpointBuilder);
	}
}

void FLiveLinkHubMessageBusSource::PostInitializeMessageEndpoint(const TSharedPtr<FMessageEndpoint>& Endpoint)
{
	if (ControlChannel->ChannelMode == FLiveLinkHubControlChannel::EChannelMode::Source)
	{
		ControlChannel->SetEndpoint(Endpoint);
	}
}

void FLiveLinkHubMessageBusSource::SendConnectMessage()
{
	UE_LOGF(LogLiveLinkHubMessaging, Verbose, "LiveLinkHubMessageBusSource (%ls): Sending connect message to %ls", *GetAddress().ToString(), *GetConnectionAddress().ToString());

	FLiveLinkHubConnectMessage* ConnectMessage = FMessageEndpoint::MakeMessage<FLiveLinkHubConnectMessage>();
	ConnectMessage->ClientInfo = ControlChannel->CreateLiveLinkClientInfo();
	ConnectMessage->ControlEndpoint = ControlChannel->GetAddress().ToString();
	ConnectMessage->SourceGuid = GetSourceId();

	TMap<FName, FString> Annotations;
	FLiveLinkMessageBusSource::AddAnnotations(Annotations);

	// Note: This needs to be sent by the MessageBus endpoint in order for the LLH Provider to be able to route messages back to it.
	MessageEndpoint->Send(ConnectMessage, EMessageFlags::None, Annotations, nullptr, { GetConnectionAddress() }, FTimespan::Zero(), FDateTime::MaxValue());

	StartHeartbeatEmitter();
	bIsValid = true;
	bIsShuttingDown = false;
}

void FLiveLinkHubMessageBusSource::InitializeAndPushStaticData_AnyThread(FName SubjectName, TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkSubjectKey& SubjectKey, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, UScriptStruct* MessageTypeInfo)
{
	check(MessageTypeInfo->IsChildOf(FLiveLinkBaseStaticData::StaticStruct()));

	FLiveLinkStaticDataStruct DataStruct(MessageTypeInfo);
	DataStruct.InitializeWith(MessageTypeInfo, reinterpret_cast<const FLiveLinkBaseStaticData*>(Context->GetMessage()));

	FLiveLinkClient::FPendingSubjectStatic PendingStaticData;
	PendingStaticData.Role = SubjectRole;
	PendingStaticData.SubjectKey = SubjectKey;
	PendingStaticData.StaticData = MoveTemp(DataStruct);
	PendingStaticData.ExtraMetadata = Context->GetAnnotations();

	static_cast<FLiveLinkClient*>(Client)->PushPendingSubject_AnyThread(MoveTemp(PendingStaticData));
}

