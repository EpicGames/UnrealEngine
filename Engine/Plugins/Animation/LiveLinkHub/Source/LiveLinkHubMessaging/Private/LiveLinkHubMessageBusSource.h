// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMessageBusSource.h"

#include "LiveLinkHubMessageBusSourceSettings.h"

class FLiveLinkHubControlChannel;
struct FMessageAddress;

/** LiveLink Message bus source that is connected to a livelink hub. */
class FLiveLinkHubMessageBusSource : public FLiveLinkMessageBusSource, public TSharedFromThis<FLiveLinkHubMessageBusSource>
{
public:
	FLiveLinkHubMessageBusSource(const FText& InSourceType, const FText& InSourceMachineName, const FMessageAddress& InConnectionAddress, double InMachineTimeOffset, int32 InDiscoveryProtocolVersion = 2);
	/** Get the ID for this source. Only valid after ReceiveClient has been invoked. */
	FGuid GetSourceId() const { return SourceGuid; }
	/** Get the address of the connected provider. */
	FMessageAddress GetConnectionAddress() const { return CachedConnectionAddress; }
	/** The control channel to use for sending control messages to the provider. */
	void SetControlChannel(TSharedPtr<FLiveLinkHubControlChannel> InControlChannel) { ControlChannel = MoveTemp(InControlChannel); }

protected:
	//~ Begin FLiveLinkMessageBusSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeAndPushStaticData_AnyThread(FName SubjectName, TSubclassOf<ULiveLinkRole> SubjectRole, const FLiveLinkSubjectKey& SubjectKey, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, UScriptStruct* MessageTypeInfo) override;
	virtual double GetDeadSourceTimeout() const override;
	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override { return ULiveLinkHubMessageBusSourceSettings::StaticClass(); }
	virtual bool RequestSourceShutdown() override;
	// This lets child classes the opportunity to add custom message handlers to the endpoint builder
	virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& EndpointBuilder);
	virtual void PostInitializeMessageEndpoint(const TSharedPtr<FMessageEndpoint>& Endpoint);
	//~ End FLiveLinkMessageBusSource

	// Send connect message to the provider and start the heartbeat emitter
	virtual void SendConnectMessage() override;

private:
	/** Cached ConnectionAddress that stays valid after a call to RequestSourceShutdown. */
	FMessageAddress CachedConnectionAddress;

	/** Used to communicate with a LLH provider built before 5.7. */
	TSharedPtr<FLiveLinkHubControlChannel> ControlChannel;

	/** Keeps track of what discovery protocol the connected provider is using. */
	int32 DiscoveryProtocolVersion = 2;
};
