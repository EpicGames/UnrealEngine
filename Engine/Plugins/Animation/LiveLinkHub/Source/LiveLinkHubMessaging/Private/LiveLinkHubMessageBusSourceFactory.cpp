// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubMessageBusSourceFactory.h"

#include "ILiveLinkHubMessagingModule.h"
#include "LiveLinkHubMessageBusSource.h"
#include "LiveLinkMessages.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "SLiveLinkMessageBusSourceFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkHubMessageBusSourceFactory)


#define LOCTEXT_NAMESPACE "LiveLinkHubMessageBusSourceFactory"


FText ULiveLinkHubMessageBusSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "Live Link Hub");
}

FText ULiveLinkHubMessageBusSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Creates a connection to a Live Link Hub instance.");
}

TSharedPtr<SWidget> ULiveLinkHubMessageBusSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkMessageBusSourceFactory)
		.OnSourceSelected(FOnLiveLinkMessageBusSourceSelected::CreateUObject(this, &ULiveLinkHubMessageBusSourceFactory::OnSourceSelected, InOnLiveLinkSourceCreated))
		.FactoryClass(GetClass());
}

TSharedPtr<FLiveLinkMessageBusSource> ULiveLinkHubMessageBusSourceFactory::MakeSource(const FText& Name,
																				   const FText& MachineName,
																				   const FMessageAddress& Address,
																				   double TimeOffset) const
{
	ILiveLinkHubMessagingModule& Module = FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");
	return MakeShared<FLiveLinkHubMessageBusSource>(Name, MachineName, Address, TimeOffset);
}

bool ULiveLinkHubMessageBusSourceFactory::IsEnabled() const
{
	ILiveLinkHubMessagingModule& Module = FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");
	return Module.GetHostTopologyMode() == ELiveLinkTopologyMode::Hub || Module.GetHostTopologyMode() == ELiveLinkTopologyMode::UnrealClient;
}


#undef LOCTEXT_NAMESPACE
