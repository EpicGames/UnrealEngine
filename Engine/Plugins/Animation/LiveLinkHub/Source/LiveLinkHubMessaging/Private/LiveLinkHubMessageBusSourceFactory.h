// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMessageBusFinder.h"		// For FProviderPollResultPtr typedef
#include "LiveLinkMessageBusSourceFactory.h"
#include "LiveLinkHubMessageBusSourceFactory.generated.h"

#define UE_API LIVELINKHUBMESSAGING_API

struct FMessageAddress;


UCLASS(MinimalAPI)
class ULiveLinkHubMessageBusSourceFactory : public ULiveLinkMessageBusSourceFactory
{
public:
	GENERATED_BODY()

	//~ Begin ULiveLinkSourceFactory interface
	UE_API virtual FText GetSourceDisplayName() const override;
	UE_API virtual FText GetSourceTooltip() const override;
	UE_API virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	UE_API virtual bool IsEnabled() const override;
	//~ End ULiveLinkSourceFactory interface

protected:
	//~ ULiveLinkMessageBusSourceFactory interface
	UE_API virtual TSharedPtr<class FLiveLinkMessageBusSource> MakeSource(const FText& Name,
																   const FText& MachineName,
																   const FMessageAddress& Address,
																   double TimeOffset) const override;
};

#undef UE_API
