// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubMessages.h"

#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/SystemTimeTimecodeProvider.h"
#include "Engine/TimecodeProvider.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkModule.h"
#include "LiveLinkCustomTimeStep.h"
#include "LiveLinkTimecodeProvider.h"
#include "TimecodeCustomTimeStep.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkHubMessages)

const FName UE::LiveLinkHub::Private::LiveLinkHubProviderType = TEXT("LiveLinkHub");
FName FLiveLinkHubMessageAnnotation::ProviderTypeAnnotation = TEXT("ProviderType");
FName FLiveLinkHubMessageAnnotation::AutoConnectModeAnnotation = TEXT("AutoConnect");
FName FLiveLinkHubMessageAnnotation::IdAnnotation = TEXT("Id");
FName FLiveLinkHubMessageAnnotation::ExplicitConnectAnnotation = TEXT("ExplicitConnect");


DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHubMessages, Log, All);


namespace UE::LiveLinkHubMessages::Private
{
	TOptional<FLiveLinkSubjectKey> FindSubjectKey(ILiveLinkClient* LiveLinkClient, FName SubjectName)
	{
		TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient->GetSubjects(true, true);
		// We need to map the named subject to the list of subject keys available.
		for (const FLiveLinkSubjectKey& Key : Subjects)
		{
			if (Key.SubjectName == SubjectName)
			{
				return Key;
			}
		}
		return {};
	}
}

void FLiveLinkHubCustomTimeStepSettings::AssignCustomTimeStepToEngine() const
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (!ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		return;
	}

	if (GEngine == nullptr)
	{
		return;
	}

	if (bResetCustomTimeStep)
	{
		UEngineCustomTimeStep* CurrentCustomTimeStep = GEngine->GetCustomTimeStep();

		// Only reset a custom time step that the hub owns, so we don't clobber one the user set in the editor.
		// Hub-owned markers: ULiveLinkHubCustomTimeStep (LiveLink-driven) and ULiveLinkHubTimecodeCustomTimeStep (Timecode-driven).
		if (Cast<ULiveLinkHubCustomTimeStep>(CurrentCustomTimeStep) || Cast<ULiveLinkHubTimecodeCustomTimeStep>(CurrentCustomTimeStep))
		{
			UE_LOGF(LogLiveLinkHubMessages, Display, "CustomTimeStep reset event");

			GEngine->Exec(GEngine->GetCurrentPlayWorld(nullptr), TEXT("CustomTimeStep.reset"));
		}
		return;
	}

	if (Kind == ELiveLinkHubCustomTimeStepKind::Timecode)
	{
		UE_LOGF(LogLiveLinkHubMessages, Display, "CustomTimeStep change event (Timecode) - MaxDeltaTime %f", MaxDeltaTime);

		ULiveLinkHubTimecodeCustomTimeStep* NewCustomTimeStep = NewObject<ULiveLinkHubTimecodeCustomTimeStep>(GEngine, ULiveLinkHubTimecodeCustomTimeStep::StaticClass());
		NewCustomTimeStep->bErrorIfFrameAreNotConsecutive = bErrorIfFrameAreNotConsecutive;
		NewCustomTimeStep->bErrorIfTimecodeProviderChanged = bErrorIfTimecodeProviderChanged;
		NewCustomTimeStep->bIgnoreSubframes = bIgnoreSubframes;
		NewCustomTimeStep->MaxDeltaTime = MaxDeltaTime;

		GEngine->SetCustomTimeStep(NewCustomTimeStep);
		return;
	}

	UE_LOGF(LogLiveLinkHubMessages, Display, "CustomTimeStep change event (LiveLink) %ls - %ls", *SubjectName.ToString(), *CustomTimeStepRate.ToPrettyText().ToString());

	ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	ULiveLinkHubCustomTimeStep* NewCustomTimeStep = NewObject<ULiveLinkHubCustomTimeStep>(GEngine, ULiveLinkHubCustomTimeStep::StaticClass());
	NewCustomTimeStep->LiveLinkDataRate = CustomTimeStepRate;
	NewCustomTimeStep->bLockStepMode = bLockStepMode;
	NewCustomTimeStep->FrameRateDivider = FrameRateDivider;

	if (TOptional<FLiveLinkSubjectKey> Target = UE::LiveLinkHubMessages::Private::FindSubjectKey(LiveLinkClient, SubjectName))
	{
		NewCustomTimeStep->SubjectKey = *Target;
	}
	else
	{
		// Note: We must set the subject name because the livelink custom time step will use it to try to match new subjects being added.
		NewCustomTimeStep->SubjectKey = FLiveLinkSubjectKey{FGuid(), SubjectName};
	}

	// Override the custom timestep for the engine.
	GEngine->SetCustomTimeStep(NewCustomTimeStep);
}

void FLiveLinkHubTimecodeSettings::AssignTimecodeSettingsAsProviderToEngine() const
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (!ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName) || GEngine == nullptr)
	{
		return;
	}

	ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	UE_LOGF(LogLiveLinkHubMessages, Display, "Time code change event %ls - %ls", *UEnum::GetValueAsName(Source).ToString(), *SubjectName.ToString());
	if (Source == ELiveLinkHubTimecodeSource::SystemTimeEditor)
	{
		// If we are using system time, construct a new system time code provider with the target framerate.
		FName ObjectName = MakeUniqueObjectName(GEngine, USystemTimeTimecodeProvider::StaticClass(), "LiveLinkHubSystemTimecodeProvider");
		USystemTimeTimecodeProvider* NewTimecodeProvider = NewObject<USystemTimeTimecodeProvider>(GEngine, ObjectName);
		NewTimecodeProvider->FrameRate = DesiredFrameRate;
		NewTimecodeProvider->FrameDelay = FrameDelay;
		GEngine->SetTimecodeProvider(NewTimecodeProvider);
		UE_LOGF(LogLiveLinkHubMessages, Display, "System Time Timecode provider set.");
	}
	else if (Source == ELiveLinkHubTimecodeSource::UseSubjectName)
	{
		TOptional<FLiveLinkSubjectKey> Target = UE::LiveLinkHubMessages::Private::FindSubjectKey(LiveLinkClient, SubjectName);

		FName ObjectName = MakeUniqueObjectName(GEngine, ULiveLinkTimecodeProvider::StaticClass(), "DefaultLiveLinkTimecodeProvider");
		ULiveLinkTimecodeProvider* LiveLinkProvider = NewObject<ULiveLinkTimecodeProvider>(GEngine, ObjectName);

		if (Target)
		{
			LiveLinkProvider->SetTargetSubjectKey(*Target);
			LiveLinkProvider->OverrideFrameRate = DesiredFrameRate;
		}
		else
		{
			// Create a mock subject key in order to match a subject when it comes online.
			LiveLinkProvider->SetTargetSubjectKey(FLiveLinkSubjectKey{ FGuid{}, SubjectName });
			UE_LOGF(LogLiveLinkHubMessages, Warning, "Assigned Timecode provider with invalid subject %ls.", *SubjectName.ToString());
		}

		LiveLinkProvider->OverrideFrameRate = DesiredFrameRate;
		LiveLinkProvider->bOverrideFrameRate = true;
		LiveLinkProvider->FrameDelay = FrameDelay;
		LiveLinkProvider->BufferSize = BufferSize;
		LiveLinkProvider->Evaluation = OverrideEvaluationType ? *OverrideEvaluationType : EvaluationType;
		GEngine->SetTimecodeProvider(LiveLinkProvider);
		UE_LOGF(LogLiveLinkHubMessages, Display, "Live Link Timecode provider assigned to %ls at %ls", *SubjectName.ToString(), *DesiredFrameRate.ToPrettyText().ToString());
	}
	else
	{
		// Force the timecode provider to reset back to the default setting.
		GEngine->Exec( GEngine->GetCurrentPlayWorld(nullptr), TEXT( "TimecodeProvider.reset" ) );
	}
}

FLiveLinkHubTimecodeSettings FLiveLinkHubTimecodeSettings::FromTimecodeProvider(const UTimecodeProvider* InTimecodeProvider)
{
	FLiveLinkHubTimecodeSettings Settings;

	if (!InTimecodeProvider)
	{
		Settings.Source = ELiveLinkHubTimecodeSource::NotDefined;
		return Settings;
	}

	if (const ULiveLinkTimecodeProvider* LiveLinkProvider = Cast<ULiveLinkTimecodeProvider>(InTimecodeProvider))
	{
		Settings.Source = ELiveLinkHubTimecodeSource::UseSubjectName;
		Settings.SubjectName = LiveLinkProvider->SubjectKey.SubjectName;
		Settings.DesiredFrameRate = LiveLinkProvider->GetQualifiedFrameTime().Rate;
		//If the user has bOverrideFrameRate = true, use the override frame rate.
		if(LiveLinkProvider->bOverrideFrameRate)
		{
			Settings.DesiredFrameRate = LiveLinkProvider->OverrideFrameRate;
		}
		Settings.FrameDelay = LiveLinkProvider->FrameDelay;
		Settings.BufferSize = LiveLinkProvider->BufferSize;
		Settings.EvaluationType = LiveLinkProvider->Evaluation;
		return Settings;
	}

	if (const USystemTimeTimecodeProvider* SystemTimeProvider = Cast<USystemTimeTimecodeProvider>(InTimecodeProvider))
	{
		Settings.Source = ELiveLinkHubTimecodeSource::SystemTimeEditor;
		Settings.DesiredFrameRate = SystemTimeProvider->FrameRate;
		Settings.FrameDelay = SystemTimeProvider->FrameDelay;
		return Settings;
	}

	UE_LOGF(LogLiveLinkHubMessages, Warning, "Unsupported timecode provider type '%ls' for LiveLinkHub broadcast. Falling back to NotDefined.", *InTimecodeProvider->GetClass()->GetName());
	Settings.Source = ELiveLinkHubTimecodeSource::NotDefined;
	return Settings;
}

FLiveLinkHubCustomTimeStepSettings FLiveLinkHubCustomTimeStepSettings::FromCustomTimeStep(const UEngineCustomTimeStep* InCustomTimeStep)
{
	FLiveLinkHubCustomTimeStepSettings Settings;

	if (!InCustomTimeStep)
	{
		Settings.Kind = ELiveLinkHubCustomTimeStepKind::Reset;
		Settings.bResetCustomTimeStep = true;
		return Settings;
	}

	if (const ULiveLinkCustomTimeStep* LiveLinkTimeStep = Cast<ULiveLinkCustomTimeStep>(InCustomTimeStep))
	{
		Settings.Kind = ELiveLinkHubCustomTimeStepKind::LiveLink;
		Settings.bResetCustomTimeStep = false;
		Settings.SubjectName = LiveLinkTimeStep->SubjectKey.SubjectName;
		Settings.CustomTimeStepRate = LiveLinkTimeStep->LiveLinkDataRate;
		Settings.bLockStepMode = LiveLinkTimeStep->bLockStepMode;
		Settings.FrameRateDivider = LiveLinkTimeStep->FrameRateDivider;
		return Settings;
	}

	if (const UTimecodeCustomTimeStep* TimecodeTimeStep = Cast<UTimecodeCustomTimeStep>(InCustomTimeStep))
	{
		Settings.Kind = ELiveLinkHubCustomTimeStepKind::Timecode;
		Settings.bResetCustomTimeStep = false;
		Settings.bErrorIfFrameAreNotConsecutive = TimecodeTimeStep->bErrorIfFrameAreNotConsecutive;
		Settings.bErrorIfTimecodeProviderChanged = TimecodeTimeStep->bErrorIfTimecodeProviderChanged;
		Settings.bIgnoreSubframes = TimecodeTimeStep->bIgnoreSubframes;
		Settings.MaxDeltaTime = TimecodeTimeStep->MaxDeltaTime;
		return Settings;
	}

	UE_LOGF(LogLiveLinkHubMessages, Warning, "Unsupported custom time step type '%ls' for LiveLinkHub broadcast. Falling back to reset.", *InCustomTimeStep->GetClass()->GetName());
	Settings.Kind = ELiveLinkHubCustomTimeStepKind::Reset;
	Settings.bResetCustomTimeStep = true;
	return Settings;
}
