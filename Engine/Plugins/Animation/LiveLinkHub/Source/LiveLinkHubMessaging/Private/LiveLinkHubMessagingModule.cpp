// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubMessagingModule.h"

#include "CoreMinimal.h"
#include "IMessageInterceptor.h"
#include "LiveLinkHubConnectionManager.h"
#include "LiveLinkHubControlChannel.h"
#include "LiveLinkHubMessageBusSourceFactory.h"
#include "LiveLinkMessageBusFinder.h"
#include "LiveLinkMessageBusSourceFactory.h"
#include "Logging/StructuredLog.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

DEFINE_LOG_CATEGORY(LogLiveLinkHubMessaging);

#define LOCTEXT_NAMESPACE "LiveLinkHubMessaging"

namespace Internal
{
	// Source: https://en.cppreference.com/w/cpp/utility/variant/visit
	template<class... Ts>
	struct TOverloaded : Ts...
	{
		using Ts::operator()...;
	};

	template<class... Ts> TOverloaded(Ts...) -> TOverloaded<Ts...>;
}

void FLiveLinkHubMessagingModule::StartupModule()
{
	const bool bIsLiveLinkHubHost = GConfig->GetBoolOrDefault(
		TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);

	InstanceInfo.TopologyMode = bIsLiveLinkHubHost ? ELiveLinkTopologyMode::Hub : ELiveLinkTopologyMode::UnrealClient;

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	
	ConnectionManager = MakePimpl<FLiveLinkHubConnectionManager>(InstanceInfo.TopologyMode,
		FLiveLinkHubConnectionManager::FOnGetTopologyMode::CreateRaw(this, &FLiveLinkHubMessagingModule::GetHostTopologyMode),
		FLiveLinkHubConnectionManager::FOnGetInstanceId::CreateRaw(this, &FLiveLinkHubMessagingModule::GetInstanceId),
		FLiveLinkHubConnectionManager::FOnDiscoveryRequest::CreateRaw(this, &FLiveLinkHubMessagingModule::OnDiscoveryRequest)
	);

	ControlChannel = MakeShared<FLiveLinkHubControlChannel>(FLiveLinkHubControlChannel::EChannelMode::Global);
	ControlChannel->OnAuxRequest.BindRaw(this, &FLiveLinkHubMessagingModule::OnAuxRequest);
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]() { ControlChannel->Initialize(); });

#endif

	RegisterSettings();

	SourceFilterDelegate = ILiveLinkModule::Get().RegisterMessageBusSourceFilter(FOnLiveLinkShouldDisplaySource::CreateRaw(this, &FLiveLinkHubMessagingModule::OnFilterMessageBusSource));
}

void FLiveLinkHubMessagingModule::ShutdownModule()
{ 
	if (ILiveLinkModule* LiveLinkModule = FModuleManager::Get().GetModulePtr<ILiveLinkModule>("LiveLink"))
	{
		LiveLinkModule->UnregisterMessageBusSourceFilter(SourceFilterDelegate);
	}

	UnregisterSettings();

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	ControlChannel->OnAuxRequest.Unbind();
	ControlChannel.Reset();
	ConnectionManager.Reset();
#endif
}

void FLiveLinkHubMessagingModule::SetHostTopologyMode(ELiveLinkTopologyMode InMode)
{
	FScopeLock Lock(&InstanceInfoLock);
	InstanceInfo.TopologyMode = InMode;
}

FLiveLinkHubInstanceId FLiveLinkHubMessagingModule::GetInstanceId() const
{
	FScopeLock Lock(&InstanceInfoLock);
	return InstanceInfo.Id;
}

void FLiveLinkHubMessagingModule::OnDiscoveryRequest(const FMessageAddress& RemoteAddress) const
{
	// Just tell the LLH instance we exist so their address book has an entry for the Control Endpoint.
	// This could probably be improved if there was some mechanism for it in MessageBus itself.
	if (ControlChannel)
	{
		ControlChannel->SendBeacon(RemoteAddress);
	}
}

void FLiveLinkHubMessagingModule::SetInstanceId(const FLiveLinkHubInstanceId& Id)
{
	FScopeLock Lock(&InstanceInfoLock);
	InstanceInfo.Id = Id;
}

ELiveLinkTopologyMode FLiveLinkHubMessagingModule::GetHostTopologyMode() const
{
	FScopeLock Lock(&InstanceInfoLock);
	return InstanceInfo.TopologyMode;
}


bool FLiveLinkHubMessagingModule::RegisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct, FAuxChannelRequestHandlerFunc&& InHandlerFunc)
{
	if (!ensure(InRequestTypeStruct))
	{
		UE_LOGFMT(LogLiveLinkHubMessaging, Error, "Attempted to register handler for null request type");
		return false;
	}

	if (!ensure(InRequestTypeStruct != FLiveLinkHubAuxChannelRequestMessage::StaticStruct()))
	{
		UE_LOGFMT(LogLiveLinkHubMessaging, Error, "Attempted to register handler for base request type");
		return false;
	}

	if (!ensure(!AuxChannelRequestHandlers.Find(InRequestTypeStruct)))
	{
		UE_LOGFMT(LogLiveLinkHubMessaging, Error, "Handler already exists for request type {RequestTypeName}", InRequestTypeStruct->GetName());
		return false;
	}

	AuxChannelRequestHandlers.Add(InRequestTypeStruct, MoveTemp(InHandlerFunc));

	return true;
}


bool FLiveLinkHubMessagingModule::UnregisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct)
{
	if (!ensure(InRequestTypeStruct))
	{
		UE_LOGFMT(LogLiveLinkHubMessaging, Error, "Attempted to unregister null request type");
		return false;
	}

	return AuxChannelRequestHandlers.Remove(InRequestTypeStruct) != 0;
}


bool FLiveLinkHubMessagingModule::OnFilterMessageBusSource(UClass* FactoryClass, TSharedPtr<struct FProviderPollResult, ESPMode::ThreadSafe> PollResult)
{
	// Only display Hub/Spoke sources in "LiveLinkHub" section of the add source dropdown.
	bool bValidTopologyMode = false;
	if (FactoryClass == ULiveLinkHubMessageBusSourceFactory::StaticClass())
	{
		bValidTopologyMode = LiveLinkHubConnectionManager::GetPollResultTopologyMode(PollResult) == ELiveLinkTopologyMode::Hub || LiveLinkHubConnectionManager::GetPollResultTopologyMode(PollResult) == ELiveLinkTopologyMode::Spoke;
	}
	else if (FactoryClass == ULiveLinkMessageBusSourceFactory::StaticClass())
	{
		bValidTopologyMode = LiveLinkHubConnectionManager::GetPollResultTopologyMode(PollResult) != ELiveLinkTopologyMode::Hub;
	}

	return bValidTopologyMode && LiveLinkHubConnectionManager::ShouldAcceptConnectionFrom(InstanceInfo.TopologyMode, PollResult, InstanceInfo.Id);
}

TSharedPtr<FLiveLinkHubControlChannel> FLiveLinkHubMessagingModule::GetControlChannel()
{
	return ControlChannel;
}

bool FLiveLinkHubMessagingModule::OnAuxRequest(
    const FLiveLinkHubAuxChannelRequestMessage& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	const UScriptStruct* MessageTypeInfo = InContext->GetMessageTypeInfo().Get();
	if (FAuxChannelRequestHandlerFunc* Handler = AuxChannelRequestHandlers.Find(MessageTypeInfo))
	{
		(*Handler)(InMessage, InContext);
		return true;
	}
	else
	{
		// Returning false generates a FLiveLinkHubAuxChannelRejectMessage response.
		return false;
	}
}


void FLiveLinkHubMessagingModule::RegisterSettings()
{
#if WITH_EDITOR
	// Only create this section if in the hub.
	if (GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni))
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Application", "Messaging",
				LOCTEXT("LiveLinkSettingsName", "Messaging"),
				LOCTEXT("LiveLinkDescription", "Configure Live Link Hub Messaging"),
				GetMutableDefault<ULiveLinkHubMessagingSettings>()
			);
		}
	}
#endif
}

void FLiveLinkHubMessagingModule::UnregisterSettings()
{
#if WITH_EDITOR
	if (GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni))
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Application", "Timing & Sync");
		}
	}
#endif
}

FLiveLinkHubInstanceId::FLiveLinkHubInstanceId(FGuid Guid)
{
	Id.Set<FGuid>(Guid);
}

FLiveLinkHubInstanceId::FLiveLinkHubInstanceId(FStringView NamedId)
{
	Id.Emplace<FString>(NamedId);
}

FString FLiveLinkHubInstanceId::ToString() const
{
	auto Overload = Internal::TOverloaded{
		[](FGuid InGuid) -> FString 
		{
			TStringBuilder<20> LiveLinkHubName;
			LiveLinkHubName << TEXT("Live Link Hub") << TEXT(" (") << *InGuid.ToString().Right(4).ToLower() << TEXT(")");
			return LiveLinkHubName.ToString();
		},
		[](const FString& InId) -> FString
		{
			return InId;
		}
	};

	return Visit(Overload, Id);
}

IMPLEMENT_MODULE(FLiveLinkHubMessagingModule, LiveLinkHubMessaging);

#undef LOCTEXT_NAMESPACE