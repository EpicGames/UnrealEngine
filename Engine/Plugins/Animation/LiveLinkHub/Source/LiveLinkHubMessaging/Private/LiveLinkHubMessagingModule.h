// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkHubMessagingModule.h"

#include "LiveLinkHubMessages.h"
#include "HAL/CriticalSection.h"

#ifndef WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
#define WITH_LIVELINK_DISCOVERY_MANAGER_THREAD 1
#endif

class FLiveLinkHubConnectionManager;
class FLiveLinkHubMessageBusSource;


class FLiveLinkHubMessagingModule : public ILiveLinkHubMessagingModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin ILiveLinkHubMessagingModule interface
	virtual FOnHubConnectionEstablished& OnConnectionEstablished() override
	{
		return ConnectionEstablishedDelegate;
	}

	virtual void SetHostTopologyMode(ELiveLinkTopologyMode InMode) override;
	virtual FLiveLinkHubInstanceId GetInstanceId() const override;
	virtual void SetInstanceId(const FLiveLinkHubInstanceId& Id) override;
	virtual ELiveLinkTopologyMode GetHostTopologyMode() const override;
	virtual bool RegisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct, FAuxChannelRequestHandlerFunc&& InHandlerFunc) override;
	virtual bool UnregisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct) override;
	//~ End ILiveLinkHubMessagingModule interface

	/** Used by the LiveLinkHubMessageBusSource to acquire a control channel when it's created. */
	TSharedPtr<class FLiveLinkHubControlChannel> GetControlChannel();

private:
	/**
	 * Note: Invoked on the UI (Game) thread.
	 * Filter invoked by the messagebus source factory to filter out sources in the creation panel.
	 */
	bool OnFilterMessageBusSource(UClass* FactoryClass, TSharedPtr<struct FProviderPollResult, ESPMode::ThreadSafe> PollResult);

	/** Handle a discovery request from LLH to populate its address book with the ControlEndpoint's address. */
	void OnDiscoveryRequest(const struct FMessageAddress& RemoteAddress) const;

	/** Handle aux channel request messages. */
	bool OnAuxRequest(const FLiveLinkHubAuxChannelRequestMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	/** Register settings related to LiveLinkHub messaging. */
	void RegisterSettings();

	/** Unregister settings related to LiveLinkHub messaging. */
	void UnregisterSettings();

private:
	bool bUseConnectionManager;

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	/** Manages the connection to the live link hub. */
	TPimplPtr<FLiveLinkHubConnectionManager> ConnectionManager;
#endif

	/** Handle to the delegate used to filter message bus sources. */
	FDelegateHandle SourceFilterDelegate;

	/** Delegate called when the connection between a livelink hub and the editor is established. */
	FOnHubConnectionEstablished ConnectionEstablishedDelegate;

	/** Lock to access the instance info struct. */
	mutable FCriticalSection InstanceInfoLock;

	struct FInstanceInfo
	{
		/** Topology Mode for this host. */
		ELiveLinkTopologyMode TopologyMode = ELiveLinkTopologyMode::Hub;
		/** Instance ID for this host. */
		FLiveLinkHubInstanceId Id = FLiveLinkHubInstanceId(FGuid());
	} InstanceInfo;

	/** Control channel used to communicate with LLH Providers built on 5.7+. */
	TSharedPtr<class FLiveLinkHubControlChannel> ControlChannel;

	TMap<const UScriptStruct*, FAuxChannelRequestHandlerFunc> AuxChannelRequestHandlers;
};

