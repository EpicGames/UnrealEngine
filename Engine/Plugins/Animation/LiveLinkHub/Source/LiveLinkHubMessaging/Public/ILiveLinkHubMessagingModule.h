// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "Misc/TVariant.h"
#include "Modules/ModuleInterface.h"

#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

#define UE_API LIVELINKHUBMESSAGING_API

enum class ELiveLinkTopologyMode : uint8;
struct FLiveLinkHubAuxChannelRequestMessage;
class IMessageContext;


DECLARE_MULTICAST_DELEGATE_OneParam(FOnHubConnectionEstablished, FGuid SourceId);

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkHubMessaging, Display, All);


class FLiveLinkHubInstanceId
{
public:
	UE_API explicit FLiveLinkHubInstanceId(FGuid Guid);
	UE_API explicit FLiveLinkHubInstanceId(FStringView NamedId);

	UE_API FString ToString() const;

private:
	/** Variant holding the actual instance ID. */
	TVariant<FGuid, FString> Id;
};


class ILiveLinkHubMessagingModule : public IModuleInterface
{
public:
	/** Delegate called when a connection is established to a livelink hub. */
	virtual FOnHubConnectionEstablished& OnConnectionEstablished() = 0;

	/** Set the topology mode for this host. This will dictate to what apps it can connect to. */
	virtual void SetHostTopologyMode(ELiveLinkTopologyMode InMode) = 0;

	/** Get the topology mode for this host. */
	virtual ELiveLinkTopologyMode GetHostTopologyMode() const = 0;

	/** Set the instance ID for this connection manager, used to detect if it's trying to connect to itself. */
	virtual void SetInstanceId(const FLiveLinkHubInstanceId& Id) = 0;

	/** Get the ID for this running instance. Returns an empty guid if this isn't running as part of LiveLinkHub. */
	virtual FLiveLinkHubInstanceId GetInstanceId() const = 0;

	using FAuxChannelRequestHandlerFunc = TUniqueFunction<void(
		const FLiveLinkHubAuxChannelRequestMessage&,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&
	)>;

	/** Register a callback to be invoked when a peer attempts to negotiate an auxiliary endpoint. */
	template<typename RequestType UE_REQUIRES(std::is_base_of_v<FLiveLinkHubAuxChannelRequestMessage, RequestType>)>
	bool RegisterAuxChannelRequestHandler(TUniqueFunction<void(const RequestType&, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)>&& InHandlerFunc)
	{
		// Type erase the handler and pass through to the virtual helper.
		return RegisterAuxChannelRequestHandler(
		    RequestType::StaticStruct(),
			[HandlerFunc = MoveTemp(InHandlerFunc)]
			(
				const FLiveLinkHubAuxChannelRequestMessage& InMessage,
				const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
			)
			{
			    const RequestType& Message = static_cast<const RequestType&>(InMessage);
				HandlerFunc(Message, InContext);
			}
		);
	}

	/** Unregister an existing auxiliary endpoint handler. */
	template<typename RequestType UE_REQUIRES(std::is_base_of_v<FLiveLinkHubAuxChannelRequestMessage, RequestType>)>
	bool UnregisterAuxChannelRequestHandler()
	{
		return UnregisterAuxChannelRequestHandler(RequestType::StaticStruct());
	}

	/** Register a callback to be invoked when a peer attempts to negotiate an auxiliary endpoint. */
	virtual bool RegisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct, FAuxChannelRequestHandlerFunc&& InHandlerFunc) = 0;

	/** Unregister an existing auxiliary endpoint handler. */
	virtual bool UnregisterAuxChannelRequestHandler(UScriptStruct* InRequestTypeStruct) = 0;

};

#undef UE_API
