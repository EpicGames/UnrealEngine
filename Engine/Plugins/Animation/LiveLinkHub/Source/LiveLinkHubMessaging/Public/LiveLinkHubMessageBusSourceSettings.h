// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMessageBusSourceSettings.h"

#include "LiveLinkHubMessageBusSourceSettings.generated.h"

/** Class used to allow customizing the default behavior of LiveLinkHub sources and more generally to allow quickly identifying if a source is a LLH instance. */
UCLASS(MinimalAPI)
class ULiveLinkHubMessageBusSourceSettings : public ULiveLinkMessageBusSourceSettings
{
public:
	GENERATED_BODY()
};
