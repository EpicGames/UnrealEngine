// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "LiveLinkHubMessages.h"

#include "LiveLinkHubMessagingSettings.generated.h"

/**
 * Settings for the messaging protocol of LiveLinkHub.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class ULiveLinkHubMessagingSettings : public UObject
{
	GENERATED_BODY()

public:

	ULiveLinkHubMessagingSettings()
	{
		using enum ELiveLinkTopologyMode;

		CanReceiveFrom_Map =
		{
			{ UnrealClient, Hub },
			{ UnrealClient, UnrealClient },
			{ UnrealClient, External },
			{ Spoke, External },
			{ Hub, External },
			{ Hub, Spoke },
		};

		CanTransmitTo_Map =
		{
			{ Hub, UnrealClient },
			{ UnrealClient, UnrealClient },
			{ Spoke, Hub }
		};
	}

	/** Whether to allow the hub to receive LiveLink data from an unreal instance. (Useful if Unreal is using Live Link Broadcast components) */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkHub")
	bool bAllowReceivingFromUnreal = false;

public:
	//~ UObject interface
	virtual void PostInitProperties() override
	{
		using enum ELiveLinkTopologyMode;

		Super::PostInitProperties();

		if (bAllowReceivingFromUnreal)
		{
			CanReceiveFrom_Map.Add(Hub, UnrealClient);
			CanTransmitTo_Map.Add(UnrealClient, Hub);
		}
	}

#if WITH_EDITOR
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
	{
		using enum ELiveLinkTopologyMode;

		UObject::PostEditChangeProperty(PropertyChangedEvent);

		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubMessagingSettings, bAllowReceivingFromUnreal))
		{
			if (bAllowReceivingFromUnreal)
			{
				CanReceiveFrom_Map.Add(Hub, UnrealClient);
				CanTransmitTo_Map.Add(UnrealClient, Hub);
			}
			else
			{
				CanReceiveFrom_Map.Remove(Hub, UnrealClient);
				CanTransmitTo_Map.Remove(UnrealClient, Hub);
			}
		}
	}
#endif
	//~ End UObject interface

	/** Whether an instance with the LHS mode can act as a Source for the RHS mode instance. */
	bool CanReceiveFrom(ELiveLinkTopologyMode LHSMode, ELiveLinkTopologyMode RHSMode) const
	{
		TArray<ELiveLinkTopologyMode> AllowedConnections;
		CanReceiveFrom_Map.MultiFind(LHSMode, AllowedConnections);

		return AllowedConnections.Contains(RHSMode);
	}

	/** Whether an instance with the LHS mode can act as a Provider for the RHS mode instance. */
	bool CanTransmitTo(ELiveLinkTopologyMode LHSMode, ELiveLinkTopologyMode RHSMode) const
	{
		TArray<ELiveLinkTopologyMode> AllowedConnections;
		CanTransmitTo_Map.MultiFind(LHSMode, AllowedConnections);
		return AllowedConnections.Contains(RHSMode);
	}

private:
	/**
	 * Map of possible connections.
	 * Left side can receive a connection from right side.
	 */
	TMultiMap<ELiveLinkTopologyMode, ELiveLinkTopologyMode> CanReceiveFrom_Map;

	/**
		* Map of possible connections.
		* Left side can transmit data to a client from right side.
		*/
	TMultiMap<ELiveLinkTopologyMode, ELiveLinkTopologyMode> CanTransmitTo_Map;
};
